// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Messaging : ModuleRules
	{
		public Messaging(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"TraceLog",
				});

			PrivateIncludePaths.AddRange(
				new string[] {
					"Runtime/Messaging/Private/Bus",
					"Runtime/Messaging/Private/Bridge",
				});
		}
	}
}
