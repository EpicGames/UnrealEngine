// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "IMessageContext.h"
#include "Misc/ScopeLock.h"

/**
 * Implements an address book that maps message addresses to remote nodes.
 */
class FMessageAddressBook
{
public:

	/** Default constructor. */
	FMessageAddressBook()
	{
		CriticalSection = new FCriticalSection();
	}

	/** Destructor. */
	~FMessageAddressBook()
	{
		delete CriticalSection;
	}

public:

	/**
	 * Adds an address to the address book.
	 *
	 * @Address The message address to add.
	 * @NodeId The identifier of the remote node that handles the message address.
	 */
	void Add(const FMessageAddress& Address, const FGuid& NodeId)
	{
		FScopeLock Lock(CriticalSection);

		Entries.FindOrAdd(Address) = NodeId;
	}

	/** Clears the address book. */
	void Clear()
	{
		FScopeLock Lock(CriticalSection);

		Entries.Reset();
	}

	/**
	 * Checks whether this address book contains the given address.
	 *
	 * @param Address The address to check.
	 * @return true if the address is known, false otherwise.
	 */
	bool Contains(const FMessageAddress& Address)
	{
		FScopeLock Lock(CriticalSection);

		return Entries.Contains(Address);
	}

	/**
	 * Gets the remote node identifiers for the specified list of message addresses.
	 *
	 * @param Addresses The address list to retrieve the node identifiers for.
	 * @return The list of node identifiers.
	 */
	TArray<FGuid> GetNodesFor(const TArray<FMessageAddress>& Addresses)
	{
		TArray<FGuid> FoundNodes;

		FScopeLock Lock(CriticalSection);

		for (const auto& Address : Addresses)
		{
			FGuid* NodeId = Entries.Find(Address);

			if (NodeId != nullptr)
			{
				FoundNodes.AddUnique(*NodeId);
			}
		}

		return FoundNodes;
	}

	/**
	 * Removes all known message addresses.
	 *
	 * To remove only the addresses for a specific remote node, use RemoveNode().
	 * If you are not interested in the removed addresses, use Clear() instead.
	 *
	 * @param OutRemovedRecipients Will hold a list of recipients that were removed.
	 * @see Clear, RemoveNode
	 */
	void RemoveAll(TArray<FMessageAddress>& OutRemovedAddresses)
	{
		OutRemovedAddresses.Reset();

		FScopeLock Lock(CriticalSection);

		Entries.GenerateKeyArray(OutRemovedAddresses);
		Entries.Reset();
	}

	/**
	 * Removes all known message addresses for the specified remote node identifier.
	 *
	 * @param NodeId The identifier of the remote node to remove.
	 * @param OutRemovedRecipients Will hold a list of recipients that were removed.
	 * @see Clear, RemoveAllNodes
	 */
	void RemoveNode(const FGuid& NodeId, TArray<FMessageAddress>& OutRemovedAddresses)
	{
		OutRemovedAddresses.Reset();

		FScopeLock Lock(CriticalSection);

		for (const auto& EntryPair : Entries)
		{
			if (EntryPair.Value == NodeId)
			{
				OutRemovedAddresses.Add(EntryPair.Key);
			}
		}

		for (const auto& Address : OutRemovedAddresses)
		{
			Entries.Remove(Address);
		}
	}

private:

	/** Holds a critical section to serialize access to the address book entries. */
	FCriticalSection* CriticalSection;

	/** Holds the collection of known addresses and their remote node identifiers. */
	TMap<FMessageAddress, FGuid> Entries;
};
