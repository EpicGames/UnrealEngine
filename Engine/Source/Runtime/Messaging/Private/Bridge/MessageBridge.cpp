// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bridge/MessageBridge.h"
#include "IMessagingModule.h"
#include "IMessageBus.h"
#include "IMessageSubscription.h"
#include "IMessageTransport.h"


/* FMessageBridge structors
 *****************************************************************************/

FMessageBridge::FMessageBridge(
	const FMessageAddress InAddress,
	const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InBus,
	const TSharedRef<IMessageTransport, ESPMode::ThreadSafe>& InTransport
)
	: Address(InAddress)
	, Bus(InBus)
	, Enabled(false)
	, Id(FGuid::NewGuid())
	, Transport(InTransport)
{
	Bus->OnShutdown().AddRaw(this, &FMessageBridge::HandleMessageBusShutdown);
}


FMessageBridge::~FMessageBridge()
{
	Disable();

	if (Bus.IsValid())
	{
		Bus->OnShutdown().RemoveAll(this);
		Bus->Unregister(Address);

		TArray<FMessageAddress> RemovedAddresses;
		AddressBook.RemoveAll(RemovedAddresses);

		for (const auto& RemovedAddress : RemovedAddresses)
		{
			Bus->Unregister(RemovedAddress);
		}
	}
}


/* IMessageBridge interface
 *****************************************************************************/

void FMessageBridge::Disable() 
{
	if (!Enabled)
	{
		return;
	}

	// disable subscription & transport
	if (MessageSubscription.IsValid())
	{
		MessageSubscription->Disable();
	}

	if (Transport.IsValid())
	{
		Transport->StopTransport();
	}

	Enabled = false;
}


void FMessageBridge::Enable()
{
	if (Enabled || !Bus.IsValid() || !Transport.IsValid())
	{
		return;
	}

	// enable subscription & transport
	if (!Transport->StartTransport(*this))
	{
		return;
	}

	Bus->Register(Address, AsShared());

	if (MessageSubscription.IsValid())
	{
		MessageSubscription->Enable();
	}
	else
	{
		MessageSubscription = Bus->Subscribe(AsShared(), IMessageBus::PATHNAME_All, FMessageScopeRange::AtLeast(EMessageScope::Network));
	}

	Enabled = true;
}


bool FMessageBridge::IsEnabled() const
{
	return Enabled;
}

FGuid FMessageBridge::LookupAddress(const FMessageAddress &InAddress)
{
	if (InAddress.IsValid())
	{
		TArray<FGuid> RemoteNodes = AddressBook.GetNodesFor({InAddress});
		ensure(RemoteNodes.Num() <= 1);
		if (RemoteNodes.Num()>0 && RemoteNodes[0].IsValid())
		{
			return RemoteNodes[0];
		}
	}
	return {};
}

bool FMessageBridge::IsSubscribed(const FTopLevelAssetPath& MessageType) const
{
	// this implementation subscribes to all message types (i.e., IMessageBus::PATHNAME_All)
	return true;
}

/* IMessageReceiver interface
 *****************************************************************************/

FName FMessageBridge::GetDebugName() const
{
	return *FString::Printf(TEXT("FMessageBridge (%s)"), *Transport->GetDebugName().ToString());
}


const FGuid& FMessageBridge::GetRecipientId() const
{
	return Id;
}


ENamedThreads::Type FMessageBridge::GetRecipientThread() const
{
	return ENamedThreads::AnyThread;
}


bool FMessageBridge::IsLocal() const
{
	return false;
}


void FMessageBridge::ReceiveMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogMessaging, Verbose, "Received %ls message from %ls", *Context->GetMessageTypePathName().ToString(), *Context->GetSender().ToString());

	if (!Enabled)
	{
		return;
	}

	// get remote nodes
	TArray<FGuid> RemoteNodes;

	if (Context->GetRecipients().Num() > 0)
	{
		RemoteNodes = AddressBook.GetNodesFor(Context->GetRecipients());

		if (RemoteNodes.Num() == 0)
		{
			return;
		}
	}

	// forward message to remote nodes
	Transport->TransportMessage(Context, RemoteNodes);
}


/* IMessageSender interface
 *****************************************************************************/

FMessageAddress FMessageBridge::GetSenderAddress()
{
	return Address;
}


void FMessageBridge::NotifyMessageError(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FString& Error)
{
	// deprecated
}


/* IMessageTransportHandler interface
 *****************************************************************************/

void FMessageBridge::DiscoverTransportNode(const FGuid& NodeId)
{
	// do nothing (address book is updated in ReceiveTransportMessage)
}


void FMessageBridge::ForgetTransportNode(const FGuid& NodeId)
{
	TArray<FMessageAddress> RemovedAddresses;

	// update address book
	AddressBook.RemoveNode(NodeId, RemovedAddresses);

	// unregister endpoints
	if (Bus.IsValid())
	{
		for (const auto& RemovedAddress : RemovedAddresses)
		{
			Bus->Unregister(RemovedAddress);
		}
	}
}


void FMessageBridge::ReceiveTransportMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FGuid& NodeId)
{
#if !NO_LOGGING
	if (UE_GET_LOG_VERBOSITY(LogMessaging) >= ELogVerbosity::Verbose)
	{
		FString RecipientStr = FString::JoinBy(Context->GetRecipients(), TEXT("+"), &FMessageAddress::ToString);
		UE_LOGF(LogMessaging, Verbose, "FMessageBridge::ReceiveTransportMessage: Received %ls from %ls for %ls",
			*Context->GetMessageTypePathName().ToString(),
			*Context->GetSender().ToString(),
			*RecipientStr);
	}
#endif

	if (!Enabled || !Bus.IsValid())
	{
		return;
	}

	// discard expired messages
	if (Context->GetExpiration() < FDateTime::UtcNow())
	{
		UE_LOGF(LogMessaging, Verbose, "FMessageBridge::ReceiveTransportMessage: Message expired. Discarding");
		return;
	}

	// register newly discovered endpoints
	if (!AddressBook.Contains(Context->GetSender()))
	{
		UE_LOGF(LogMessaging, Verbose, "FMessageBridge::ReceiveTransportMessage: Registering new sender %ls", *Context->GetMessageTypePathName().ToString());

		AddressBook.Add(Context->GetSender(), NodeId);
		Bus->Register(Context->GetSender(), AsShared());
	}

	// forward message to local bus
	Bus->Forward(Context, Context->GetRecipients(), FTimespan::Zero(), AsShared());
}


/* FMessageBridge callbacks
 *****************************************************************************/

void FMessageBridge::HandleMessageBusShutdown()
{
	Disable();
	Bus.Reset();
}
