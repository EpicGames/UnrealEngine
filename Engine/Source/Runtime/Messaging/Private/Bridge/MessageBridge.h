// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageBridge.h"
#include "IMessageContext.h"
#include "IMessageReceiver.h"
#include "IMessageSender.h"
#include "IMessageTransportHandler.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"

#include "Bridge/MessageAddressBook.h"

class IMessageBus;
class IMessageSubscription;
class IMessageTransport;


/**
 * Implements a message bridge.
 *
 * A message bridge is a special message endpoint that connects multiple message buses
 * running in different processes or on different devices. This allows messages that are
 * available in one system to also be available on other systems.
 *
 * Message bridges use an underlying transport layer to channel the messages between two
 * or more systems. Such layers may utilize system specific technologies, such as network
 * sockets or shared memory to communicate with remote bridges. The bridge acts as a map
 * from message addresses to remote nodes and vice versa.
 *
 * @see IMessageBus, IMessageTransport
 */
class FMessageBridge
	: public TSharedFromThis<FMessageBridge, ESPMode::ThreadSafe>
	, public IMessageBridge
	, public IMessageReceiver
	, public IMessageSender
	, protected IMessageTransportHandler
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InAddress The message address for this bridge.
	 * @param InBus The message bus that this node is connected to.
	 * @param InTransport The transport mechanism to use.
	 */
	FMessageBridge(
		const FMessageAddress InAddress,
		const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InBus,
		const TSharedRef<IMessageTransport, ESPMode::ThreadSafe>& InTransport
	);

	/** Virtual destructor. */
	virtual ~FMessageBridge();

public:

	//~ IMessageBridge interface

	virtual void Disable() override;
	virtual void Enable() override;
	virtual bool IsEnabled() const override;
	virtual FGuid LookupAddress(const FMessageAddress& InAddress) override;
	virtual bool IsSubscribed(const FTopLevelAssetPath& MessageType) const override;

	//~ IMessageReceiver interface

	virtual FName GetDebugName() const override;
	virtual const FGuid& GetRecipientId() const override;
	virtual ENamedThreads::Type GetRecipientThread() const override;
	virtual bool IsLocal() const override;
	virtual void ReceiveMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) override;

public:

	//~ IMessageSender interface

	virtual FMessageAddress GetSenderAddress() override;
	virtual void NotifyMessageError(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FString& Error) override;

protected:

	//~ IMessageTransportHandler interface

	virtual void DiscoverTransportNode(const FGuid& NodeId) override;
	virtual void ForgetTransportNode(const FGuid& NodeId) override;
	virtual void ReceiveTransportMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FGuid& NodeId) override;

private:

	/** Callback for message bus shutdowns. */
	void HandleMessageBusShutdown();

private:

	/** Holds the bridge's address. */
	FMessageAddress Address;

	/** Holds the address book. */
	FMessageAddressBook AddressBook;

	/** Holds a reference to the bus that this bridge is attached to. */
	TSharedPtr<IMessageBus, ESPMode::ThreadSafe> Bus;

	/** Hold a flag indicating whether this endpoint is active. */
	bool Enabled;

	/** Holds the bridge's unique identifier (for debugging purposes). */
	const FGuid Id;

	/** Holds the message subscription for outbound messages. */
	TSharedPtr<IMessageSubscription, ESPMode::ThreadSafe> MessageSubscription;

	/** Holds the message transport object. */
	TSharedPtr<IMessageTransport, ESPMode::ThreadSafe> Transport;
};
