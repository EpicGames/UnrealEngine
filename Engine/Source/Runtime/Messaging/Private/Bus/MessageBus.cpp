// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bus/MessageBus.h"
#include "IMessagingModule.h"
#include "HAL/RunnableThread.h"
#include "Bus/MessageRouter.h"
#include "Bus/MessageContext.h"
#include "Bus/MessageSubscription.h"
#include "IMessageSender.h"
#include "IMessageReceiver.h"
#include "HAL/ThreadSingleton.h"



const FTopLevelAssetPath IMessageBus::PATHNAME_All(TEXT("/Unknown"), NAME_All);

/* FMessageBus structors
 *****************************************************************************/

FMessageBus::FMessageBus(FString InName, const TSharedPtr<IAuthorizeMessageRecipients>& InRecipientAuthorizer)
	: Name(MoveTemp(InName))
	, RecipientAuthorizer(InRecipientAuthorizer)
{
	Router = new FMessageRouter();
	RouterThread = FRunnableThread::Create(Router, *FString::Printf(TEXT("FMessageBus.%s.Router"), *Name), 128 * 1024, TPri_Normal, FPlatformAffinity::GetPoolThreadMask());

	check(Router != nullptr);
}


FMessageBus::~FMessageBus()
{
	Shutdown();

	delete Router;
}


/* IMessageBus interface
 *****************************************************************************/

void FMessageBus::Forward(
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
	const TArray<FMessageAddress>& Recipients,
	const FTimespan& Delay,
	const TSharedRef<IMessageSender, ESPMode::ThreadSafe>& Forwarder
)
{
#if !NO_LOGGING
	if (UE_GET_LOG_VERBOSITY(LogMessaging) >= ELogVerbosity::Verbose)
	{
		FString RecipientStr = FString::JoinBy(Context->GetRecipients(), TEXT("+"), &FMessageAddress::ToString);

		UE_LOGF(LogMessaging, Verbose, "Forwarding %ls from %ls to %ls",
			*Context->GetMessageTypePathName().ToString(),
			*Context->GetSender().ToString(), *RecipientStr);
	}
#endif

	Router->RouteMessage(MakeShareable(new FMessageContext(
		Context,
		Forwarder->GetSenderAddress(),
		Recipients,
		EMessageScope::Process,
		FDateTime::UtcNow() + Delay,
		FTaskGraphInterface::Get().GetCurrentThreadIfKnown()
	)));
}


TSharedRef<IMessageTracer, ESPMode::ThreadSafe> FMessageBus::GetTracer()
{
	return Router->GetTracer();
}


void FMessageBus::Intercept(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType)
{
	if (MessageType.IsNull())
	{
		return;
	}

	if (!RecipientAuthorizer.IsValid() || RecipientAuthorizer->AuthorizeInterceptor(Interceptor, MessageType))
	{
		UE_LOGF(LogMessaging, Verbose, "Adding invterceptor %ls", *Interceptor->GetDebugName().ToString());
		Router->AddInterceptor(Interceptor, MessageType);
	}			
}


FOnMessageBusShutdown& FMessageBus::OnShutdown()
{
	return ShutdownDelegate;
}


void FMessageBus::Publish(
	void* Message,
	UScriptStruct* TypeInfo,
	EMessageScope Scope,
	const TMap<FName, FString>& Annotations,
	const FTimespan& Delay,
	const FDateTime& Expiration,
	const TSharedRef<IMessageSender, ESPMode::ThreadSafe>& Publisher
)
{
	UE_LOGF(LogMessaging, Verbose, "Publishing %ls from sender %ls", *TypeInfo->GetName(), *Publisher->GetSenderAddress().ToString());

	Router->RouteMessage(MakeShared<FMessageContext, ESPMode::ThreadSafe>(
		Message,
		TypeInfo,
		Annotations,
		nullptr,
		Publisher->GetSenderAddress(),
		TArray<FMessageAddress>(),
		Scope,
		EMessageFlags::None,
		FDateTime::UtcNow() + Delay,
		Expiration,
		FTaskGraphInterface::Get().GetCurrentThreadIfKnown()
	));
}


void FMessageBus::Register(const FMessageAddress& Address, const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Recipient)
{
	UE_LOGF(LogMessaging, Verbose, "Registering %ls", *Address.ToString());
	Router->AddRecipient(Address, Recipient);
}


void FMessageBus::Send(
	void* Message,
	UScriptStruct* TypeInfo,
	EMessageFlags Flags,
	const TMap<FName, FString>& Annotations,
	const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment,
	const TArray<FMessageAddress>& Recipients,
	const FTimespan& Delay,
	const FDateTime& Expiration,
	const TSharedRef<IMessageSender, ESPMode::ThreadSafe>& Sender
)
{
	UE_LOGF(LogMessaging, Verbose, "Sending %ls to %d recipients", *TypeInfo->GetName(), Recipients.Num());

	Router->RouteMessage(MakeShared<FMessageContext, ESPMode::ThreadSafe>(
		Message,
		TypeInfo,
		Annotations,
		Attachment,
		Sender->GetSenderAddress(),
		Recipients,
		EMessageScope::Network,
		Flags,
		FDateTime::UtcNow() + Delay,
		Expiration,
		FTaskGraphInterface::Get().GetCurrentThreadIfKnown()
	));
}


void FMessageBus::Shutdown()
{
	if (RouterThread != nullptr)
	{
		ShutdownDelegate.Broadcast();

		RouterThread->Kill(true);
		delete RouterThread;
		RouterThread = nullptr;
	}
}


TSharedPtr<IMessageSubscription, ESPMode::ThreadSafe> FMessageBus::Subscribe(
	const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber,
	const FTopLevelAssetPath& MessageType,
	const FMessageScopeRange& ScopeRange
)
{
	if (!MessageType.IsNull())
	{
		if (!RecipientAuthorizer.IsValid() || RecipientAuthorizer->AuthorizeSubscription(Subscriber, MessageType))
		{
			UE_LOGF(LogMessaging, Verbose, "Subscribing %ls", *Subscriber->GetDebugName().ToString());
			TSharedRef<IMessageSubscription, ESPMode::ThreadSafe> Subscription = MakeShareable(new FMessageSubscription(Subscriber, MessageType, ScopeRange));
			Router->AddSubscription(Subscription);

			return Subscription;
		}
	}

	return nullptr;
}


void FMessageBus::Unintercept(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType)
{
	if (!MessageType.IsNull())
	{
		UE_LOGF(LogMessaging, Verbose, "Unintercepting %ls", *Interceptor->GetDebugName().ToString());
		Router->RemoveInterceptor(Interceptor, MessageType);
	}
}


void FMessageBus::Unregister(const FMessageAddress& Address)
{
	UE_LOGF(LogMessaging, Verbose, "Attempting to unregister %ls", *Address.ToString());
	if (!RecipientAuthorizer.IsValid() || RecipientAuthorizer->AuthorizeUnregistration(Address))
	{
		UE_LOGF(LogMessaging, Verbose, "Unregistered %ls", *Address.ToString());
		Router->RemoveRecipient(Address);
	}
}


void FMessageBus::Unsubscribe(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FTopLevelAssetPath& MessageType)
{
	if (!MessageType.IsNull())
	{
		if (!RecipientAuthorizer.IsValid() || RecipientAuthorizer->AuthorizeUnsubscription(Subscriber, MessageType))
		{
			UE_LOGF(LogMessaging, Verbose, "Unsubscribing %ls", *Subscriber->GetDebugName().ToString());
			Router->RemoveSubscription(Subscriber, MessageType);
		}
	}
}

void FMessageBus::AddNotificationListener(const TSharedRef<IBusListener, ESPMode::ThreadSafe>& Listener)
{
	Router->AddNotificationListener(Listener);
}

void FMessageBus::RemoveNotificationListener(const TSharedRef<IBusListener, ESPMode::ThreadSafe>& Listener)
{
	Router->RemoveNotificationListener(Listener);
}

const FString& FMessageBus::GetName() const
{
	return Name;
}

// IMessageBus deprecated functions

void IMessageBus::Intercept(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FName& MessageType)
{
	Intercept(Interceptor, UClass::TryConvertShortTypeNameToPathName<UStruct>(MessageType.ToString()));
}

TSharedPtr<IMessageSubscription, ESPMode::ThreadSafe> IMessageBus::Subscribe(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FName& MessageType, const TRange<EMessageScope>& ScopeRange)
{
	return Subscribe(Subscriber, UClass::TryConvertShortTypeNameToPathName<UStruct>(MessageType.ToString()), ScopeRange);
}

void IMessageBus::Unintercept(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FName& MessageType)
{
	Unintercept(Interceptor, UClass::TryConvertShortTypeNameToPathName<UStruct>(MessageType.ToString()));
}

void IMessageBus::Unsubscribe(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FName& MessageType)
{
	Unsubscribe(Subscriber, UClass::TryConvertShortTypeNameToPathName<UStruct>(MessageType.ToString()));
}