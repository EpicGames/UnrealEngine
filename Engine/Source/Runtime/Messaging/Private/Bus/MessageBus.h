// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageContext.h"
#include "IMessageSubscription.h"
#include "IMessageAttachment.h"
#include "IMessageInterceptor.h"
#include "IAuthorizeMessageRecipients.h"
#include "IMessageTracer.h"
#include "IMessageBus.h"

class FMessageRouter;
class IMessageReceiver;
class IMessageSender;

/**
 * Implements a message bus.
 */
class FMessageBus
	: public TSharedFromThis<FMessageBus, ESPMode::ThreadSafe>
	, public IMessageBus
{
public:
	
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InDebugName The debug name of this message bus.
	 * @param InRecipientAuthorizer An optional recipient authorizer.
	 */
	FMessageBus(FString InName, const TSharedPtr<IAuthorizeMessageRecipients>& InRecipientAuthorizer);

	/** Virtual destructor. */
	virtual ~FMessageBus();

public:

	//~ IMessageBus interface

	virtual void Forward(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay, const TSharedRef<IMessageSender, ESPMode::ThreadSafe>& Forwarder) override;
	virtual TSharedRef<IMessageTracer, ESPMode::ThreadSafe> GetTracer() override;
	virtual void Intercept(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType) override;
	virtual FOnMessageBusShutdown& OnShutdown() override;
	virtual void Publish(void* Message, UScriptStruct* TypeInfo, EMessageScope Scope, const TMap<FName, FString>& Annotations, const FTimespan& Delay, const FDateTime& Expiration, const TSharedRef<IMessageSender, ESPMode::ThreadSafe>& Publisher) override;
	virtual void Register(const FMessageAddress& Address, const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Recipient) override;
	virtual void Send(void* Message, UScriptStruct* TypeInfo, EMessageFlags Flags, const TMap<FName, FString>& Annotations, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay, const FDateTime& Expiration, const TSharedRef<IMessageSender, ESPMode::ThreadSafe>& Sender) override;
	virtual void Shutdown() override;
	virtual TSharedPtr<IMessageSubscription, ESPMode::ThreadSafe> Subscribe(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FTopLevelAssetPath& MessageType, const FMessageScopeRange& ScopeRange) override;
	virtual void Unintercept(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType) override;
	virtual void Unregister(const FMessageAddress& Address) override;
	virtual void Unsubscribe(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FTopLevelAssetPath& MessageType) override;

	virtual void AddNotificationListener(const TSharedRef<IBusListener, ESPMode::ThreadSafe>& Listener) override;
	virtual void RemoveNotificationListener(const TSharedRef<IBusListener, ESPMode::ThreadSafe>& Listener) override;
	virtual const FString& GetName() const override;

private:
	/** The message bus debugging name. */
	const FString Name;

	/** Holds the message router. */
	FMessageRouter* Router;

	/** Holds the message router thread. */
	FRunnableThread* RouterThread;

	/** Holds the recipient authorizer. */
	TSharedPtr<IAuthorizeMessageRecipients> RecipientAuthorizer;

	/** Holds bus shutdown delegate. */
	FOnMessageBusShutdown ShutdownDelegate;
};
