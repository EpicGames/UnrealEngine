// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bus/MessageContext.h"


/* FMessageContext structors
 *****************************************************************************/

FMessageContext::~FMessageContext()
{
	if (Message != nullptr)
	{
		if (UScriptStruct* TypeInfoPtr = TypeInfo.Get())
		{
			TypeInfoPtr->DestroyStruct(Message);
		}

		FMemory::Free(Message);
	}
}


/* IMessageContext interface
 *****************************************************************************/

const TMap<FName, FString>& FMessageContext::GetAnnotations() const
{
	if (OriginalContext.IsValid())
	{
		return OriginalContext->GetAnnotations();
	}

	return Annotations;
}


TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> FMessageContext::GetAttachment() const
{
	if (OriginalContext.IsValid())
	{
		return OriginalContext->GetAttachment();
	}

	return Attachment;
}


const FDateTime& FMessageContext::GetExpiration() const
{
	if (OriginalContext.IsValid())
	{
		return OriginalContext->GetExpiration();
	}

	return Expiration;
}


const void* FMessageContext::GetMessage() const
{
	if (OriginalContext.IsValid())
	{
		return OriginalContext->GetMessage();
	}

	return Message;
}


const TWeakObjectPtr<UScriptStruct>& FMessageContext::GetMessageTypeInfo() const
{
	if (OriginalContext.IsValid())
	{
		return OriginalContext->GetMessageTypeInfo();
	}

	return TypeInfo;
}


TSharedPtr<IMessageContext, ESPMode::ThreadSafe> FMessageContext::GetOriginalContext() const
{
	return OriginalContext;
}

const TArray<FMessageAddress>& FMessageContext::GetRecipients() const
{
	return Recipients;
}


EMessageScope FMessageContext::GetScope() const
{
	return Scope;
}

EMessageFlags FMessageContext::GetFlags() const
{
	if (OriginalContext.IsValid())
	{
		return OriginalContext->GetFlags();
	}

	return Flags;
}

const FMessageAddress& FMessageContext::GetSender() const
{
	if (OriginalContext.IsValid())
	{
		return OriginalContext->GetSender();
	}

	return Sender;
}


const FMessageAddress& FMessageContext::GetForwarder() const
{
	return Sender;
}


ENamedThreads::Type FMessageContext::GetSenderThread() const
{
	return SenderThread;
}


const FDateTime& FMessageContext::GetTimeForwarded() const
{
	return TimeSent;
}


const FDateTime& FMessageContext::GetTimeSent() const
{
	if (OriginalContext.IsValid())
	{
		return OriginalContext->GetTimeSent();
	}

	return TimeSent;
}
