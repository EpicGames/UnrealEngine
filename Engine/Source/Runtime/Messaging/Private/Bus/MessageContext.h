// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageContext.h"
#include "IMessageAttachment.h"

/**
 * Implements a message context for messages sent through the message bus.
 *
 * Message contexts contain a message and additional data about that message,
 * such as when the message was sent, who sent it and where it is being sent to.
 */

class FMessageContext
	: public IMessageContext
{
public:

	/** Default constructor. */
	FMessageContext()
		: Message(nullptr)
		, TypeInfo(nullptr)
	{ }

	/**
	 * Creates and initializes a new message context.
	 *
	 * This constructor overload is used for published and sent messages.
	 *
	 * @param InMessage The message payload.
	 * @param InTypeInfo The message's type information.
	 * @param InAnnotation The message header to attach to the message.
	 * @param InAttachment The binary data to attach to the message.
	 * @param InSender The sender's address.
	 * @param InRecipients The message recipients.
	 * @param InScope The message scope.
	 * @param InFlags The message flags.
	 * @param InTimeSent The time at which the message was sent.
	 * @param InExpiration The message's expiration time.
	 * @param InSenderThread The name of the thread from which the message was sent.
	 */
	FMessageContext(
		void* InMessage,
		UScriptStruct* InTypeInfo,
		const TMap<FName, FString>& InAnnotations,
		const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& InAttachment,
		const FMessageAddress& InSender,
		const TArray<FMessageAddress>& InRecipients,
		EMessageScope InScope,
		EMessageFlags InFlags,
		const FDateTime& InTimeSent,
		const FDateTime& InExpiration,
		ENamedThreads::Type InSenderThread
	)
		: Annotations(InAnnotations)
		, Attachment(InAttachment)
		, Expiration(InExpiration)
		, Message(InMessage)
		, Recipients(InRecipients)
		, Scope(InScope)
		, Flags(InFlags)
		, Sender(InSender)
		, SenderThread(InSenderThread)
		, TimeSent(InTimeSent)
		, TypeInfo(InTypeInfo)
	{ }

	/**
	 * Creates and initializes a new message context from an existing context.
	 *
	 * This constructor overload is used for forwarded messages.
	 *
	 * @param InContext The existing context.
	 * @param InForwarder The forwarder's address.
	 * @param NewRecipients The recipients of the new context.
	 * @param NewScope The message's new scope.
	 * @param InTimeForwarded The time at which the message was forwarded.
	 * @param InForwarderThread The name of the thread from which the message was forwarded.
	 */
	FMessageContext(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext,
		const FMessageAddress& InForwarder,
		const TArray<FMessageAddress>& NewRecipients,
		EMessageScope NewScope,
		const FDateTime& InTimeForwarded,
		ENamedThreads::Type InForwarderThread
	)
		: Message(nullptr)
		, OriginalContext(InContext)
		, Recipients(NewRecipients)
		, Scope(NewScope)
		, Flags(EMessageFlags::None)
		, Sender(InForwarder)
		, SenderThread(InForwarderThread)
		, TimeSent(InTimeForwarded)
	{ }

	/** Destructor. */
	virtual ~FMessageContext() override;

public:

	//~ IMessageContext interface

	virtual const TMap<FName, FString>& GetAnnotations() const override;
	virtual TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> GetAttachment() const override;
	virtual const FDateTime& GetExpiration() const override;
	virtual const void* GetMessage() const override;
	virtual const TWeakObjectPtr<UScriptStruct>& GetMessageTypeInfo() const override;
	virtual TSharedPtr<IMessageContext, ESPMode::ThreadSafe> GetOriginalContext() const override;
	virtual const TArray<FMessageAddress>& GetRecipients() const override;
	virtual EMessageScope GetScope() const override;
	virtual EMessageFlags GetFlags() const override;
	virtual const FMessageAddress& GetSender() const override;
	virtual const FMessageAddress& GetForwarder() const override;
	virtual ENamedThreads::Type GetSenderThread() const override;
	virtual const FDateTime& GetTimeForwarded() const override;
	virtual const FDateTime& GetTimeSent() const override;

private:

	/** Holds the optional message annotations. */
	TMap<FName, FString> Annotations;

	/** Holds a pointer to attached binary data. */
	TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> Attachment;

	/** Holds the expiration time. */
	FDateTime Expiration;

	/** Holds the message. */
	void* Message;

	/** Holds the original message context. */
	TSharedPtr<IMessageContext, ESPMode::ThreadSafe> OriginalContext;

	/** Holds the message recipients. */
	TArray<FMessageAddress> Recipients;

	/** Holds the message's scope. */
	EMessageScope Scope;

	/** Holds the message's scope. */
	EMessageFlags Flags;

	/** Holds the sender's identifier. */
	FMessageAddress Sender;

	/** Holds the name of the thread from which the message was sent. */
	ENamedThreads::Type SenderThread;

	/** Holds the time at which the message was sent. */
	FDateTime TimeSent;

	/** Holds the message's type information. */
	TWeakObjectPtr<UScriptStruct> TypeInfo;
};
