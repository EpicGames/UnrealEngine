// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bus/MessageDispatchTask.h"
#include "IMessageReceiver.h"


/* FMessageDispatchTask structors
 *****************************************************************************/

FMessageDispatchTask::FMessageDispatchTask(
	ENamedThreads::Type InThread,
	TSharedRef<IMessageContext, ESPMode::ThreadSafe> InContext,
	TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe> InRecipient,
	TSharedPtr<FMessageTracer, ESPMode::ThreadSafe> InTracer
)
	: Context(InContext)
	, RecipientPtr(InRecipient)
	, Thread(InThread)
	, TracerPtr(InTracer)
{ }


/* FMessageDispatchTask interface
 *****************************************************************************/

void FMessageDispatchTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	TSharedPtr<IMessageReceiver, ESPMode::ThreadSafe> Recipient = RecipientPtr.Pin();

	if (!Recipient.IsValid())
	{
		return;
	}

	auto Tracer = TracerPtr.Pin();

	if (Tracer.IsValid())
	{
		Tracer->TraceDispatchedMessage(Context, Recipient.ToSharedRef(), true);
	}

	Recipient->ReceiveMessage(Context);

	if (TracerPtr.IsValid())
	{
		Tracer->TraceHandledMessage(Context, Recipient.ToSharedRef());
	}
}

TStatId FMessageDispatchTask::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FMessageDispatchTask, STATGROUP_TaskGraphTasks);
}

/* FBusNotificationDispatchTask interface
 *****************************************************************************/

void FBusNotificationDispatchTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	TSharedPtr<IBusListener, ESPMode::ThreadSafe> Listener = ListenerPtr.Pin();

	if (!Listener.IsValid())
	{
		return;
	}

	Listener->NotifyRegistration(Address, Notification);
}

TStatId FBusNotificationDispatchTask::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FBusNotificationDispatchTask, STATGROUP_TaskGraphTasks);
}
