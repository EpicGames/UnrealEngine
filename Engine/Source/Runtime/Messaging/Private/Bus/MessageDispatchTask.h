// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "IMessageContext.h"
#include "IMessageBusListener.h"
#include "Bus/MessageTracer.h"

class IMessageReceiver;

/**
 * Implements an asynchronous task for dispatching a message to a recipient.
 */
class FMessageDispatchTask
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InThread The name of the thread to dispatch the message on.
	 * @param InContext The context of the message to dispatch.
	 * @param InRecipient The message recipient.
	 * @param InTracer The message tracer to notify.
	 */
	FMessageDispatchTask(
		ENamedThreads::Type InThread,
		TSharedRef<IMessageContext, ESPMode::ThreadSafe> InContext,
		TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe> InRecipient,
		TSharedPtr<FMessageTracer, ESPMode::ThreadSafe> InTracer);

public:

	/**
	 * Performs the actual task.
	 *
	 * @param CurrentThread The thread that this task is executing on.
	 * @param MyCompletionGraphEvent The completion event.
	 */
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
	
	/**
	 * Returns the name of the thread that this task should run on.
	 *
	 * @return Thread name.
	 */
	ENamedThreads::Type GetDesiredThread()
	{
		return Thread;
	}

	/**
	 * Gets the task's stats tracking identifier.
	 *
	 * @return Stats identifier.
	 */
	TStatId GetStatId() const;

	/**
	 * Gets the mode for tracking subsequent tasks.
	 *
	 * @return Always track subsequent tasks.
	 */
	static ESubsequentsMode::Type GetSubsequentsMode() 
	{ 
		return ESubsequentsMode::TrackSubsequents; 
	}

private:

	/** Holds the message context. */
	TSharedRef<IMessageContext, ESPMode::ThreadSafe> Context;

	/** Holds a reference to the recipient. */
	TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe> RecipientPtr;

	/** Holds the name of the thread that the router is running on. */
	ENamedThreads::Type Thread;

	/** Holds a pointer to the message tracer. */
	TWeakPtr<FMessageTracer, ESPMode::ThreadSafe> TracerPtr;
};


/**
 * Implements an asynchronous task for dispatching a registration notification to a listener.
 */
class FBusNotificationDispatchTask
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InThread The name of the thread to dispatch the message on.
	 * @param InListener The listener to notify.
	 * @param InAddress The address we are notifying about.
	 * @param InNotification The notification type being notified.
	 */
	FBusNotificationDispatchTask(
		ENamedThreads::Type InThread,
		TWeakPtr<IBusListener, ESPMode::ThreadSafe> InListener,
		FMessageAddress InAddress,
		EMessageBusNotification InNotification)
		: Thread(InThread)
		, ListenerPtr(InListener)
		, Address(InAddress)
		, Notification(InNotification)
	{}

public:

	/**
	 * Performs the actual task.
	 *
	 * @param CurrentThread The thread that this task is executing on.
	 * @param MyCompletionGraphEvent The completion event.
	 */
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);

	/**
	 * Returns the name of the thread that this task should run on.
	 *
	 * @return Thread name.
	 */
	ENamedThreads::Type GetDesiredThread()
	{
		return Thread;
	}

	TStatId GetStatId() const;

	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

private:
	/** Holds the name of the thread that the router is running on. */
	ENamedThreads::Type Thread;

	/** Holds a reference to the recipient. */
	TWeakPtr<IBusListener, ESPMode::ThreadSafe> ListenerPtr;

	/** Holds the address we are notifying about. */
	FMessageAddress Address;

	/** Holds the notification type we are notifying about. */
	EMessageBusNotification Notification;
};
