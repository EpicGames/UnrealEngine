// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bus/MessageRouter.h"
#include "IMessagingModule.h"
#include "HAL/PlatformProcess.h"
#include "Bus/MessageDispatchTask.h"
#include "IMessageBus.h"
#include "IMessageSubscription.h"
#include "IMessageReceiver.h"
#include "IMessageInterceptor.h"
#include "IMessageBusListener.h"
#include "Misc/ConfigCacheIni.h"


/* FMessageRouter structors
 *****************************************************************************/

FMessageRouter::FMessageRouter()
	: DelayedMessagesSequence(0)
	, Stopping(false)
	, Tracer(MakeShared<FMessageTracer, ESPMode::ThreadSafe>())
	, bAllowDelayedMessaging(false)
{
	ActiveSubscriptions.FindOrAdd(IMessageBus::PATHNAME_All);
	WorkEvent = FPlatformProcess::GetSynchEventFromPool();

	GConfig->GetBool(TEXT("Messaging"), TEXT("bAllowDelayedMessaging"), bAllowDelayedMessaging, GEngineIni);
}


FMessageRouter::~FMessageRouter()
{
	FPlatformProcess::ReturnSynchEventToPool(WorkEvent);
	WorkEvent = nullptr;
}


/* FRunnable interface
 *****************************************************************************/

FSingleThreadRunnable* FMessageRouter::GetSingleThreadInterface()
{
	return this;
}


bool FMessageRouter::Init()
{
	return true;
}


uint32 FMessageRouter::Run()
{
	while (!Stopping)
	{
		CurrentTime = FDateTime::UtcNow();

		ProcessCommands();
		ProcessDelayedMessages();

		WorkEvent->Wait(CalculateWaitTime());
	}

	return 0;
}


void FMessageRouter::Stop()
{
	Tracer->Stop();
	Stopping = true;
	WorkEvent->Trigger();
}


void FMessageRouter::Exit()
{
	TArray<TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe>> Recipients;

	// gather all subscribed and registered recipients
	for (const auto& RecipientPair : ActiveRecipients)
	{
		Recipients.AddUnique(RecipientPair.Value);
	}

	for (const auto& SubscriptionsPair : ActiveSubscriptions)
	{
		for (const auto& Subscription : SubscriptionsPair.Value)
		{
			Recipients.AddUnique(Subscription->GetSubscriber());
		}
	}
}


/* FMessageRouter implementation
 *****************************************************************************/

FTimespan FMessageRouter::CalculateWaitTime()
{
	FTimespan WaitTime = FTimespan::FromMilliseconds(100);

	if (DelayedMessages.Num() > 0)
	{
		FTimespan DelayedTime = DelayedMessages.HeapTop().Context->GetTimeSent() - CurrentTime;

		if (DelayedTime < WaitTime)
		{
			return DelayedTime;
		}
	}

	return WaitTime;
}


void FMessageRouter::DispatchMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (Context->IsValid())
	{
		TArray<TSharedPtr<IMessageReceiver, ESPMode::ThreadSafe>> Recipients;

		int32 RecipientCount = Context->GetRecipients().Num();

		// get recipients, either from the context...
		if (RecipientCount > 0)
		{

#if !NO_LOGGING
			if (UE_GET_LOG_VERBOSITY(LogMessaging) >= ELogVerbosity::Verbose)
			{
				FString RecipientStr = FString::JoinBy(Context->GetRecipients(), TEXT("+"), &FMessageAddress::ToString);
				UE_LOGF(LogMessaging, Verbose, "Dispatching %ls from %ls to %ls", *Context->GetMessageTypePathName().ToString(), *Context->GetSender().ToString(), *RecipientStr);
			}
#endif

			FilterRecipients(Context, Recipients);

			if (Recipients.Num() < RecipientCount)
			{
				UE_LOGF(LogMessaging, Verbose, "%d recipients were filtered out", RecipientCount - Recipients.Num());
			}
		}
		// ... or from subscriptions
		else
		{
			FilterSubscriptions(ActiveSubscriptions.FindOrAdd(Context->GetMessageTypePathName()), Context, Recipients);
			FilterSubscriptions(ActiveSubscriptions.FindOrAdd(IMessageBus::PATHNAME_All), Context, Recipients);

			UE_SUPPRESS(LogMessaging, Verbose,
			{
				const FString RecipientStr = FString::JoinBy(Recipients, TEXT("+"),
					[](const TSharedPtr<IMessageReceiver>& Receiver)
					{
						return Receiver ? Receiver->GetRecipientId().ToString() : TEXT("");
					});

				UE_LOGF(LogMessaging, Verbose, "Dispatching %ls from %ls to %i subscriber(s): %ls"
					, *Context->GetMessageTypePathName().ToString()
					, *Context->GetSender().ToString()
					, Recipients.Num(),
					*RecipientStr);
			});
		}

		// dispatch the message
		for (auto& Recipient : Recipients)
		{
			ENamedThreads::Type RecipientThread = Recipient->GetRecipientThread();

			if (RecipientThread == ENamedThreads::AnyThread)
			{
				Tracer->TraceDispatchedMessage(Context, Recipient.ToSharedRef(), false);
				Recipient->ReceiveMessage(Context);
				Tracer->TraceHandledMessage(Context, Recipient.ToSharedRef());
			}
			else
			{
				TGraphTask<FMessageDispatchTask>::CreateTask().ConstructAndDispatchWhenReady(RecipientThread, Context, Recipient, Tracer);
			}
		}
	}
}


void FMessageRouter::FilterSubscriptions(
	TArray<TSharedPtr<IMessageSubscription, ESPMode::ThreadSafe>>& Subscriptions,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
	TArray<TSharedPtr<IMessageReceiver, ESPMode::ThreadSafe>>& OutRecipients
)
{
	EMessageScope MessageScope = Context->GetScope();

	for (int32 SubscriptionIndex = 0; SubscriptionIndex < Subscriptions.Num(); ++SubscriptionIndex)
	{
		const auto Subscription = Subscriptions[SubscriptionIndex];

		if (!Subscription->IsEnabled() || !Subscription->GetScopeRange().Contains(MessageScope))
		{
			continue;
		}

		auto Subscriber = Subscription->GetSubscriber().Pin();

		if (Subscriber.IsValid())
		{
			if (MessageScope == EMessageScope::Thread)
			{
				ENamedThreads::Type RecipientThread = Subscriber->GetRecipientThread();
				ENamedThreads::Type SenderThread = Context->GetSenderThread();

				if (RecipientThread != SenderThread)
				{
					continue;
				}
			}

			OutRecipients.AddUnique(Subscriber);
		}
		else
		{
			Subscriptions.RemoveAtSwap(SubscriptionIndex);
			--SubscriptionIndex;
		}
	}
}


void FMessageRouter::FilterRecipients(
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
	TArray<TSharedPtr<IMessageReceiver, ESPMode::ThreadSafe>>& OutRecipients)
{
	FMessageScopeRange IncludeNetwork = FMessageScopeRange::AtLeast(EMessageScope::Network);
	const TArray<FMessageAddress>& RecipientList = Context->GetRecipients();
	for (const auto& RecipientAddress : RecipientList)
	{
		auto Recipient = ActiveRecipients.FindRef(RecipientAddress).Pin();

		if (Recipient.IsValid())
		{
			// if the recipient is not local and the scope does not include network, filter it out of the recipient list
			if (Recipient->IsLocal() || IncludeNetwork.Contains(Context->GetScope()))
			{
				OutRecipients.AddUnique(Recipient);
			}
		}
		else
		{
			ActiveRecipients.Remove(RecipientAddress);
		}
	}
}


void FMessageRouter::ProcessCommands()
{
	CommandDelegate Command;

	while (Commands.Dequeue(Command))
	{
		Command.Execute();
	}
}


void FMessageRouter::ProcessDelayedMessages()
{
	FDelayedMessage DelayedMessage;

	while ((DelayedMessages.Num() > 0) && (DelayedMessages.HeapTop().Context->GetTimeSent() <= CurrentTime))
	{
		DelayedMessages.HeapPop(DelayedMessage);
		DispatchMessage(DelayedMessage.Context.ToSharedRef());
	}
}


/* FSingleThreadRunnable interface
 *****************************************************************************/

void FMessageRouter::Tick()
{
	CurrentTime = FDateTime::UtcNow();

	ProcessDelayedMessages();
	ProcessCommands();
}


/* FMessageRouter callbacks
 *****************************************************************************/

void FMessageRouter::HandleAddInterceptor(TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe> Interceptor, FTopLevelAssetPath MessageType)
{
	UE_LOGF(LogMessaging, Verbose, "Adding %ls as intereceptor for %ls messages", *Interceptor->GetDebugName().ToString(), *MessageType.ToString());

	ActiveInterceptors.FindOrAdd(MessageType).AddUnique(Interceptor);
	Tracer->TraceAddedInterceptor(Interceptor, MessageType);
}


void FMessageRouter::HandleAddRecipient(FMessageAddress Address, TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe> RecipientPtr)
{
	auto Recipient = RecipientPtr.Pin();

	if (Recipient.IsValid())
	{
		UE_LOGF(LogMessaging, Verbose, "Adding %ls on %ls as recipient", *Recipient->GetDebugName().ToString(), *Address.ToString());

		ActiveRecipients.FindOrAdd(Address) = Recipient;
		Tracer->TraceAddedRecipient(Address, Recipient.ToSharedRef());
		NotifyRegistration(Address, EMessageBusNotification::Registered);
	}
}


void FMessageRouter::HandleAddSubscriber(TSharedRef<IMessageSubscription, ESPMode::ThreadSafe> Subscription)
{
	auto Subscriber = Subscription->GetSubscriber().Pin();

	if (Subscriber.IsValid())
	{
		UE_LOGF(LogMessaging, Verbose, "Adding %ls as a subscriber for %ls messages", *Subscriber->GetDebugName().ToString(), *Subscription->GetMessageTypePathName().ToString());
	}

	ActiveSubscriptions.FindOrAdd(Subscription->GetMessageTypePathName()).AddUnique(Subscription);
	Tracer->TraceAddedSubscription(Subscription);
}


void FMessageRouter::HandleRemoveInterceptor(TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe> Interceptor, FTopLevelAssetPath MessageType)
{
	UE_LOGF(LogMessaging, Verbose, "Removing %ls as intereceptor for %ls messages", *Interceptor->GetDebugName().ToString(), *MessageType.ToString());

	if (MessageType == IMessageBus::PATHNAME_All)
	{
		for (auto& InterceptorsPair : ActiveInterceptors)
		{
			InterceptorsPair.Value.Remove(Interceptor);
		}
	}
	else
	{
		auto& Interceptors = ActiveInterceptors.FindOrAdd(MessageType);
		Interceptors.Remove(Interceptor);
	}

	Tracer->TraceRemovedInterceptor(Interceptor, MessageType);
}

void FMessageRouter::HandleRemoveRecipient(FMessageAddress Address)
{
	auto Recipient = ActiveRecipients.FindRef(Address).Pin();

	if (Recipient.IsValid())
	{
		UE_LOGF(LogMessaging, Verbose, "Removing %ls on %ls as recipient", *Recipient->GetDebugName().ToString(), *Address.ToString());

		ActiveRecipients.Remove(Address);
		Tracer->TraceRemovedRecipient(Address);
		NotifyRegistration(Address, EMessageBusNotification::Unregistered);
	}
}


void FMessageRouter::HandleRemoveSubscriber(TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe> SubscriberPtr, FTopLevelAssetPath MessageType)
{
	auto Subscriber = SubscriberPtr.Pin();

	if (!Subscriber.IsValid())
	{
		return;
	}

	for (auto& SubscriptionsPair : ActiveSubscriptions)
	{
		if ((MessageType != IMessageBus::PATHNAME_All) && (MessageType != SubscriptionsPair.Key))
		{
			continue;
		}

		TArray<TSharedPtr<IMessageSubscription, ESPMode::ThreadSafe>>& Subscriptions = SubscriptionsPair.Value;

		for (int32 SubscriptionIndex = 0; SubscriptionIndex < Subscriptions.Num(); ++SubscriptionIndex)
		{
			const auto Subscription = Subscriptions[SubscriptionIndex];

			if (Subscription->GetSubscriber().Pin() == Subscriber)
			{
				UE_LOGF(LogMessaging, Verbose, "Removing %ls as a subscriber for %ls messages", *Subscriber->GetDebugName().ToString(), *Subscription->GetMessageTypePathName().ToString());

				Subscriptions.RemoveAtSwap(SubscriptionIndex);
				Tracer->TraceRemovedSubscription(Subscription.ToSharedRef(), MessageType);

				break;
			}
		}
	}
}


void FMessageRouter::HandleRouteMessage(TSharedRef<IMessageContext, ESPMode::ThreadSafe> Context)
{
	UE_LOGF(LogMessaging, Verbose, "Routing %ls message from %ls", *Context->GetMessageTypePathName().ToString(), *Context->GetSender().ToString());

	Tracer->TraceRoutedMessage(Context);

	// intercept routing
	auto& Interceptors = ActiveInterceptors.FindOrAdd(Context->GetMessageTypePathName());

	for (auto& Interceptor : Interceptors)
	{
		if (Interceptor->InterceptMessage(Context))
		{
			UE_LOGF(LogMessaging, Verbose, "Message was intercepted by %ls", *Interceptor->GetDebugName().ToString());

			Tracer->TraceInterceptedMessage(Context, Interceptor.ToSharedRef());

			return;
		}
	}

	// dispatch the message
	if (bAllowDelayedMessaging && (Context->GetTimeSent() > CurrentTime))
	{
		UE_LOGF(LogMessaging, Verbose, "Queued message for dispatch");

		DelayedMessages.HeapPush(FDelayedMessage(Context, ++DelayedMessagesSequence));
	}
	else
	{
		DispatchMessage(Context);
	}
}

void FMessageRouter::HandleAddListener(TWeakPtr<IBusListener, ESPMode::ThreadSafe> ListenerPtr)
{
	ActiveRegistrationListeners.AddUnique(ListenerPtr);
}

void FMessageRouter::HandleRemoveListener(TWeakPtr<IBusListener, ESPMode::ThreadSafe> ListenerPtr)
{
	ActiveRegistrationListeners.Remove(ListenerPtr);
}

void FMessageRouter::NotifyRegistration(const FMessageAddress& Address, EMessageBusNotification Notification)
{
	for (auto It = ActiveRegistrationListeners.CreateIterator(); It; ++It)
	{
		auto Listener = It->Pin();
		if (Listener.IsValid())
		{
			ENamedThreads::Type ListenerThread = Listener->GetListenerThread();

			if (ListenerThread == ENamedThreads::AnyThread)
			{
				Listener->NotifyRegistration(Address, Notification);
			}
			else
			{
				TGraphTask<FBusNotificationDispatchTask>::CreateTask().ConstructAndDispatchWhenReady(ListenerThread, Listener, Address, Notification);
			}
		}
		else
		{
			It.RemoveCurrent();
		}
	}
}