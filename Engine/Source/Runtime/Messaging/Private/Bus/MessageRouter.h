// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "HAL/Runnable.h"
#include "Misc/SingleThreadRunnable.h"
#include "Templates/Atomic.h"
#include "Containers/MpscQueue.h"

#include "IMessageContext.h"
#include "IMessageTracer.h"
#include "Bus/MessageTracer.h"

class IMessageInterceptor;
class IMessageReceiver;
class IMessageSubscription;
class IBusListener;

enum class EMessageBusNotification : uint8;

/**
 * Implements a topic-based message router.
 */
class FMessageRouter
	: public FRunnable
	, private FSingleThreadRunnable
{
	DECLARE_DELEGATE(CommandDelegate)

public:

	/** Default constructor. */
	FMessageRouter();

	/** Destructor. */
	~FMessageRouter();

public:

	/**
	 * Adds a message interceptor.
	 *
	 * @param Interceptor The interceptor to add.
	 * @param MessageType The type of messages to intercept.
	 */
	FORCEINLINE void AddInterceptor(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType)
	{
		EnqueueCommand(FSimpleDelegate::CreateRaw(this, &FMessageRouter::HandleAddInterceptor, Interceptor, MessageType));
	}

	/**
	 * Adds a recipient.
	 *
	 * @param Address The address of the recipient to add.
	 * @param Recipient The recipient.
	 */
	FORCEINLINE void AddRecipient(const FMessageAddress& Address, const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Recipient)
	{
		EnqueueCommand(FSimpleDelegate::CreateRaw(this, &FMessageRouter::HandleAddRecipient, Address, TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe>(Recipient)));
	}

	/**
	 * Adds a subscription.
	 *
	 * @param Subscription The subscription to add.
	 */
	FORCEINLINE void AddSubscription(const TSharedRef<IMessageSubscription, ESPMode::ThreadSafe>& Subscription)
	{
		EnqueueCommand(FSimpleDelegate::CreateRaw(this, &FMessageRouter::HandleAddSubscriber, Subscription));
	}

	/**
	 * Gets the message tracer.
	 *
	 * @return Weak pointer to the message tracer.
	 */
	FORCEINLINE TSharedRef<IMessageTracer, ESPMode::ThreadSafe> GetTracer()
	{
		return Tracer;
	}

	/**
	 * Removes a message interceptor.
	 *
	 * @param Interceptor The interceptor to remove.
	 * @param MessageType The type of messages to stop intercepting.
	 */
	FORCEINLINE void RemoveInterceptor(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType)
	{
		EnqueueCommand(FSimpleDelegate::CreateRaw(this, &FMessageRouter::HandleRemoveInterceptor, Interceptor, MessageType));
	}

	/**
	 * Removes a recipient.
	 *
	 * @param Address The address of the recipient to remove.
	 */
	FORCEINLINE void RemoveRecipient(const FMessageAddress& Address)
	{
		EnqueueCommand(FSimpleDelegate::CreateRaw(this, &FMessageRouter::HandleRemoveRecipient, Address));
	}

	/**
	 * Removes a subscription.
	 *
	 * @param Subscriber The subscriber to stop routing messages to.
	 * @param MessageType The type of message to unsubscribe from (NAME_None = all types).
	 */
	FORCEINLINE void RemoveSubscription(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FTopLevelAssetPath& MessageType)
	{
		EnqueueCommand(FSimpleDelegate::CreateRaw(this, &FMessageRouter::HandleRemoveSubscriber, TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe>(Subscriber), MessageType));
	}

	/**
	 * Routes a message to the specified recipients.
	 *
	 * @param Context The context of the message to route.
	 */
	FORCEINLINE void RouteMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
	{
		Tracer->TraceSentMessage(Context);
		EnqueueCommand(FSimpleDelegate::CreateRaw(this, &FMessageRouter::HandleRouteMessage, Context));
	}

	/**
	 * Add a listener to the bus registration events
	 * 
	 * @param Listener The listener to as to the registration notifications
	 */
	FORCEINLINE void AddNotificationListener(const TSharedRef<IBusListener, ESPMode::ThreadSafe>& Listener)
	{
		EnqueueCommand(FSimpleDelegate::CreateRaw(this, &FMessageRouter::HandleAddListener, TWeakPtr<IBusListener, ESPMode::ThreadSafe>(Listener)));
	}

	/**
	 * Remove a listener to the bus registration events
	 *
	 * @param Listener The listener to remove from the registration notifications
	 */
	FORCEINLINE void RemoveNotificationListener(const TSharedRef<IBusListener, ESPMode::ThreadSafe>& Listener)
	{
		EnqueueCommand(FSimpleDelegate::CreateRaw(this, &FMessageRouter::HandleRemoveListener, TWeakPtr<IBusListener, ESPMode::ThreadSafe>(Listener)));
	}

public:

	//~ FRunnable interface

	virtual FSingleThreadRunnable* GetSingleThreadInterface() override;
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;

protected:

	/**
	 * Calculates the time that the thread will wait for new work.
	 *
	 * @return Wait time.
	 */
	FTimespan CalculateWaitTime();

	/**
	 * Queues up a router command.
	 *
	 * @param Command The command to queue up.
	 * @return true if the command was enqueued, false otherwise.
	 */
	FORCEINLINE bool EnqueueCommand(CommandDelegate Command)
	{
		Commands.Enqueue(Command);
		WorkEvent->Trigger();

		return true;
	}

	/**
	 * Filters a collection of subscriptions using the given message context.
	 *
	 * @param Subscriptions The subscriptions to filter.
	 * @param Context The message context to filter by.
	 * @param OutRecipients Will hold the collection of recipients.
	 */
	void FilterSubscriptions(
		TArray<TSharedPtr<IMessageSubscription, ESPMode::ThreadSafe>>& Subscriptions,
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
		TArray<TSharedPtr<IMessageReceiver, ESPMode::ThreadSafe>>& OutRecipients);

	/**
	 * Filters recipients from the given message context to gather actual recipients.
	 *
	 * @param Context The message context to filter by.
	 * @param OutRecipients Will hold the collection of recipients.
	 */
	void FilterRecipients(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
		TArray<TSharedPtr<IMessageReceiver, ESPMode::ThreadSafe>>& OutRecipients);

	/**
	 * Dispatches a single message to its recipients.
	 *
	 * @param Message The message to dispatch.
	 */
	void DispatchMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Message);

	/**
	 * Process all queued commands.
	 *
	 * @see ProcessDelayedMessages
	 */
	void ProcessCommands();

	/**
	 * Processes all delayed messages.
	 *
	 * @see ProcessCommands
	 */
	void ProcessDelayedMessages();

protected:

	//~ FSingleThreadRunnable interface

	virtual void Tick() override;

private:

	/** Structure for delayed messages. */
	struct FDelayedMessage
	{
		/** Holds the context of the delayed message. */
		TSharedPtr<IMessageContext, ESPMode::ThreadSafe> Context;

		/** Holds a sequence number used by the delayed message queue. */
		int64 Sequence;


		/** Default constructor. */
		FDelayedMessage() { }

		/** Creates and initializes a new instance. */
		FDelayedMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext, int64 InSequence)
			: Context(InContext)
			, Sequence(InSequence)
		{ }

		/** Comparison operator for heap sorting. */
		bool operator<(const FDelayedMessage& Other) const
		{
			const FTimespan Difference = Other.Context->GetTimeSent() - Context->GetTimeSent();

			if (Difference.IsZero())
			{
				return (Sequence < Other.Sequence);
			}

			return (Difference > FTimespan::Zero());
		}
	};

private:

	/** Handles adding message interceptors. */
	void HandleAddInterceptor(TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe> Interceptor, FTopLevelAssetPath MessageType);

	/** Handles adding message recipients. */
	void HandleAddRecipient(FMessageAddress Address, TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe> RecipientPtr);

	/** Handles adding of subscriptions. */
	void HandleAddSubscriber(TSharedRef<IMessageSubscription, ESPMode::ThreadSafe> Subscription);

	/** Handles the removal of message interceptors. */
	void HandleRemoveInterceptor(TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe> Interceptor, FTopLevelAssetPath MessageType);

	/** Handles the removal of message recipients. */
	void HandleRemoveRecipient(FMessageAddress Address);

	/** Handles the removal of subscribers. */
	void HandleRemoveSubscriber(TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe> SubscriberPtr, FTopLevelAssetPath MessageType);

	/** Handles the routing of messages. */
	void HandleRouteMessage(TSharedRef<IMessageContext, ESPMode::ThreadSafe> Context);

	/** Handles the addition of a listener. */
	void HandleAddListener(TWeakPtr<IBusListener, ESPMode::ThreadSafe> ListenerPtr);

	/** Handles the removal of a listener. */
	void HandleRemoveListener(TWeakPtr<IBusListener, ESPMode::ThreadSafe> ListenerPtr);

	/** Notify listeners about registration */
	void NotifyRegistration(const FMessageAddress& Address, EMessageBusNotification Notification);

private:

	/** Maps message types to interceptors. */
	TMap<FTopLevelAssetPath, TArray<TSharedPtr<IMessageInterceptor, ESPMode::ThreadSafe>>> ActiveInterceptors;

	/** Maps message addresses to recipients. */
	TMap<FMessageAddress, TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe>> ActiveRecipients;

	/** Maps message types to subscriptions. */
	TMap<FTopLevelAssetPath, TArray<TSharedPtr<IMessageSubscription, ESPMode::ThreadSafe>>> ActiveSubscriptions;

	/** Array of active registration listeners. */
	TArray<TWeakPtr<IBusListener, ESPMode::ThreadSafe>> ActiveRegistrationListeners;

	/** Holds the router command queue. */
	TMpscQueue<CommandDelegate> Commands;

	/** Holds the current time. */
	FDateTime CurrentTime;

	/** Holds the collection of delayed messages. */
	TArray<FDelayedMessage> DelayedMessages;

	/** Holds a sequence number for delayed messages. */
	int64 DelayedMessagesSequence;

	/** Holds a flag indicating that the thread is stopping. */
	TAtomic<bool> Stopping;

	/** Holds the message tracer. */
	TSharedRef<FMessageTracer, ESPMode::ThreadSafe> Tracer;

	/** Holds an event signaling that work is available. */
	FEvent* WorkEvent;

	/** Whether or not to allow delayed messaging */
	bool bAllowDelayedMessaging;
};
