// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageContext.h"
#include "IMessageSubscription.h"

class IMessageReceiver;

/**
 * Implements a message subscription.
 *
 * Message subscriptions are used by the message router to determine where to
 * dispatch published messages to. Message subscriptions are created per message
 * type.
 */
class FMessageSubscription
	: public IMessageSubscription
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSubscriber The message subscriber.
	 * @param InMessageType The type of messages to subscribe to.
	 * @param InReceivingThread The thread on which to receive messages on.
	 * @param InScopeRange The message scope range to subscribe to.
	 */
	FMessageSubscription(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& InSubscriber, const FTopLevelAssetPath& InMessageType, const FMessageScopeRange& InScopeRange)
		: Enabled(true)
		, MessageType(InMessageType)
		, ScopeRange(InScopeRange)
		, Subscriber(InSubscriber)
	{ }

public:

	//~ IMessageSubscription interface

	virtual void Disable() override
	{
		Enabled = false;
	}

	virtual void Enable() override
	{
		Enabled = true;
	}

	virtual FTopLevelAssetPath GetMessageTypePathName() override
	{
		return MessageType;
	}

	virtual const FMessageScopeRange& GetScopeRange() override
	{
		return ScopeRange;
	}

	virtual const TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe>& GetSubscriber() override
	{
		return Subscriber;
	}

	virtual bool IsEnabled() override
	{
		return Enabled;
	}

private:

	/** Holds a flag indicating whether this subscription is enabled. */
	bool Enabled;

	/** Holds the type of subscribed messages. */
	FTopLevelAssetPath MessageType;

	/** Holds the range of message scopes to subscribe to. */
	FMessageScopeRange ScopeRange;

	/** Holds the subscriber. */
	TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe> Subscriber;
};
