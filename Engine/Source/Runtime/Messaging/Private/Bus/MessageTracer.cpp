// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bus/MessageTracer.h"
#include "Containers/Ticker.h"
#include "HAL/PlatformProcess.h"
#include "IMessageInterceptor.h"
#include "IMessageReceiver.h"
#include "IMessageTracerBreakpoint.h"

/* FMessageTracer structors
 *****************************************************************************/

FMessageTracer::FMessageTracer()
	: Breaking(false)
	, ResetPending(false)
	, Running(false)
{
	ContinueEvent = FPlatformProcess::GetSynchEventFromPool();
	TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FMessageTracer::Tick), 0.0f);
}


FMessageTracer::~FMessageTracer()
{
	FTSTicker::RemoveTicker(TickDelegateHandle);
	FPlatformProcess::ReturnSynchEventToPool(ContinueEvent);
	ContinueEvent = nullptr;
}


/* FMessageTracer interface
 *****************************************************************************/

void FMessageTracer::TraceAddedInterceptor(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType)
{
	double Timestamp = FPlatformTime::Seconds();

	Traces.Enqueue([this, Interceptor, Timestamp]() {
		// create interceptor information
		auto& InterceptorInfo = Interceptors.FindOrAdd(Interceptor->GetInterceptorId());

		if (!InterceptorInfo.IsValid())
		{
			InterceptorInfo = MakeShareable(new FMessageTracerInterceptorInfo());
		}

		// initialize interceptor information
		InterceptorInfo->Name = Interceptor->GetDebugName();
		InterceptorInfo->TimeRegistered = Timestamp;
		InterceptorInfo->TimeUnregistered = 0;
	});
}


void FMessageTracer::TraceAddedRecipient(const FMessageAddress& Address, const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Recipient)
{
	double Timestamp = FPlatformTime::Seconds();

	Traces.Enqueue([this, Address, Recipient, Timestamp]() {
		// create endpoint information
		TSharedPtr<FMessageTracerEndpointInfo>& EndpointInfo = RecipientsToEndpointInfos.FindOrAdd(Recipient->GetRecipientId());

		if (!EndpointInfo.IsValid())
		{
			EndpointInfo = MakeShareable(new FMessageTracerEndpointInfo());
		}

		// initialize endpoint information
		TSharedRef<FMessageTracerAddressInfo> AddressInfo = MakeShareable(new FMessageTracerAddressInfo());
		{
			AddressInfo->Address = Address;
			AddressInfo->TimeRegistered = Timestamp;
			AddressInfo->TimeUnregistered = 0;
		}

		EndpointInfo->AddressInfos.Add(Address, AddressInfo);
		EndpointInfo->Name = Recipient->GetDebugName();
		EndpointInfo->Remote = Recipient->IsRemote();

		// add to address table
		AddressesToEndpointInfos.Add(Address, EndpointInfo);
	});
}


void FMessageTracer::TraceAddedSubscription(const TSharedRef<IMessageSubscription, ESPMode::ThreadSafe>& Subscription)
{
	if (!Running)
	{
		return;
	}

	double Timestamp = FPlatformTime::Seconds();

	Traces.Enqueue([=]() {
		// @todo gmp: trace added subscriptions
	});
}


void FMessageTracer::TraceDispatchedMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Recipient, bool Async)
{
	if (!Running)
	{
		return;
	}

	double Timestamp = FPlatformTime::Seconds();

	Traces.Enqueue([this, Context, Recipient, Async, Timestamp]() {
		// look up message & endpoint info
		TSharedPtr<FMessageTracerMessageInfo> MessageInfo = MessageInfos.FindRef(Context);

		if (!MessageInfo.IsValid())
		{
			return;
		}

		TSharedPtr<FMessageTracerEndpointInfo>& EndpointInfo = RecipientsToEndpointInfos.FindOrAdd(Recipient->GetRecipientId());

		if (!EndpointInfo.IsValid())
		{
			return;
		}

		// update message information
		TSharedRef<FMessageTracerDispatchState> DispatchState = MakeShareable(new FMessageTracerDispatchState());
		{
			DispatchState->DispatchLatency = Timestamp - MessageInfo->TimeSent;
			DispatchState->DispatchType = Async ? EMessageTracerDispatchTypes::TaskGraph : EMessageTracerDispatchTypes::Direct;
			DispatchState->EndpointInfo = EndpointInfo;
			DispatchState->RecipientThread = Recipient->GetRecipientThread();
			DispatchState->TimeDispatched = Timestamp;
			DispatchState->TimeHandled = 0.0;
		}

		MessageInfo->DispatchStates.Add(EndpointInfo, DispatchState);

		// update database
		EndpointInfo->ReceivedMessages.Add(MessageInfo);
	});
}


void FMessageTracer::TraceHandledMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Recipient)
{
	if (!Running)
	{
		return;
	}

	double Timestamp = FPlatformTime::Seconds();

	Traces.Enqueue([this, Context, Recipient, Timestamp]() {
		// look up message & endpoint info
		TSharedPtr<FMessageTracerMessageInfo> MessageInfo = MessageInfos.FindRef(Context);

		if (!MessageInfo.IsValid())
		{
			return;
		}

		TSharedPtr<FMessageTracerEndpointInfo> EndpointInfo = RecipientsToEndpointInfos.FindRef(Recipient->GetRecipientId());

		if (!EndpointInfo.IsValid())
		{
			return;
		}

		// update message information
		TSharedPtr<FMessageTracerDispatchState> DispatchState = MessageInfo->DispatchStates.FindRef(EndpointInfo);

		if (DispatchState.IsValid())
		{
			DispatchState->TimeHandled = Timestamp;
		}
	});
}


void FMessageTracer::TraceInterceptedMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor)
{
	if (!Running)
	{
		return;
	}

	double Timestamp = FPlatformTime::Seconds();

	Traces.Enqueue([this, Context, Interceptor, Timestamp]() {
		// look up message & interceptor info
		auto MessageInfo = MessageInfos.FindRef(Context);

		if (!MessageInfo.IsValid())
		{
			return;
		}

		MessageInfo->Intercepted = true;

		auto InterceptorInfo = Interceptors.FindRef(Interceptor->GetInterceptorId());

		if (!InterceptorInfo.IsValid())
		{
			return;
		}

		// update interceptor information
		InterceptorInfo->InterceptedMessages.Add(MessageInfo);
	});		
}


void FMessageTracer::TraceRemovedInterceptor(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType)
{
	double Timestamp = FPlatformTime::Seconds();

	Traces.Enqueue([this, Interceptor, MessageType, Timestamp]() {
		auto InterceptorInfo = Interceptors.FindRef(Interceptor->GetInterceptorId());

		if (!InterceptorInfo.IsValid())
		{
			return;
		}

		// update interceptor information
		InterceptorInfo->TimeUnregistered = Timestamp;
	});
}


void FMessageTracer::TraceRemovedRecipient(const FMessageAddress& Address)
{
	double Timestamp = FPlatformTime::Seconds();

	Traces.Enqueue([this, Address, Timestamp]() {
		TSharedPtr<FMessageTracerEndpointInfo> EndpointInfo = AddressesToEndpointInfos.FindRef(Address);

		if (!EndpointInfo.IsValid())
		{
			return;
		}

		// update endpoint information
		TSharedPtr<FMessageTracerAddressInfo> AddressInfo = EndpointInfo->AddressInfos.FindRef(Address);

		if (AddressInfo.IsValid())
		{
			AddressInfo->TimeUnregistered = Timestamp;
		}
	});
}


void FMessageTracer::TraceRemovedSubscription(const TSharedRef<IMessageSubscription, ESPMode::ThreadSafe>& Subscription, const FTopLevelAssetPath& MessageType)
{
	if (!Running)
	{
		return;
	}

	double Timestamp = FPlatformTime::Seconds();

	Traces.Enqueue([this, Subscription, MessageType]() {
		// @todo gmp: trace removed message subscriptions
	});
}


void FMessageTracer::TraceRoutedMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (!Running)
	{
		return;
	}

	if (ShouldBreak(Context))
	{
		Breaking = true;
		ContinueEvent->Wait();
	}

	double Timestamp = FPlatformTime::Seconds();

	Traces.Enqueue([this, Context, Timestamp]() {
		// update message information
		TSharedPtr<FMessageTracerMessageInfo> MessageInfo = MessageInfos.FindRef(Context);

		if (MessageInfo.IsValid())
		{
			MessageInfo->TimeRouted = Timestamp;
		}
	});
}


void FMessageTracer::TraceSentMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (!Running)
	{
		return;
	}

	double Timestamp = FPlatformTime::Seconds();

	Traces.Enqueue([this, Context, Timestamp]() {
		// look up endpoint info
		TSharedPtr<FMessageTracerEndpointInfo> EndpointInfo = AddressesToEndpointInfos.FindRef(Context->GetSender());

		if (!EndpointInfo.IsValid())
		{
			return;
		}

		// create message info
		TSharedRef<FMessageTracerMessageInfo> MessageInfo = MakeShareable(new FMessageTracerMessageInfo());
		{
			MessageInfo->Context = Context;
			MessageInfo->Intercepted = false;
			MessageInfo->SenderInfo = EndpointInfo;
			MessageInfo->TimeRouted = 0.0;
			MessageInfo->TimeSent = Timestamp;
			MessageInfos.Add(Context, MessageInfo);
		}

		// add message type
		TSharedPtr<FMessageTracerTypeInfo>& TypeInfo = MessageTypes.FindOrAdd(Context->GetMessageTypePathName());

		if (!TypeInfo.IsValid())
		{
			TypeInfo = MakeShareable(new FMessageTracerTypeInfo());
			TypeInfo->TypePathName = Context->GetMessageTypePathName();
			TypeAddedDelegate.Broadcast(TypeInfo.ToSharedRef());
		}

		TypeInfo->Messages.Add(MessageInfo);

		// update database
		EndpointInfo->SentMessages.Add(MessageInfo);
		MessageInfo->TypeInfo = TypeInfo;

		MessagesAddedDelegate.Broadcast(MessageInfo);
	});
}


/* IMessageTracer interface
 *****************************************************************************/

void FMessageTracer::Break()
{
	Breaking = true;
}


void FMessageTracer::Continue()
{
	if (!Running)
	{
		Running = true;
	}
	else if (Breaking)
	{
		Breaking = false;
		ContinueEvent->Trigger();
	}
}


int32 FMessageTracer::GetEndpoints(TArray<TSharedPtr<FMessageTracerEndpointInfo>>& OutEndpoints) const
{
	RecipientsToEndpointInfos.GenerateValueArray(OutEndpoints);

	return OutEndpoints.Num();
}


int32 FMessageTracer::GetMessages(TArray<TSharedPtr<FMessageTracerMessageInfo>>& OutMessages) const
{
	MessageInfos.GenerateValueArray(OutMessages);

	return OutMessages.Num();
}


int32 FMessageTracer::GetMessageTypes(TArray<TSharedPtr<FMessageTracerTypeInfo>>& OutTypes) const
{
	MessageTypes.GenerateValueArray(OutTypes);

	return OutTypes.Num();
}


bool FMessageTracer::HasMessages() const
{
	return (MessageInfos.Num() > 0);
}


bool FMessageTracer::IsBreaking() const
{
	return Breaking;
}


bool FMessageTracer::IsRunning() const
{
	return Running;
}


void FMessageTracer::Reset()
{
	ResetPending = true;
}


void FMessageTracer::Step()
{
	if (!Breaking)
	{
		return;
	}

	ContinueEvent->Trigger();
}


void FMessageTracer::Stop()
{
	if (!Running)
	{
		return;
	}

	Running = false;

	if (Breaking)
	{
		Breaking = false;
		ContinueEvent->Trigger();
	}
}


bool FMessageTracer::Tick(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageTracer_Tick);

	if (ResetPending)
	{
		ResetMessages();
		ResetPending = false;
	}

	// process new traces
	if (!Traces.IsEmpty())
	{
		TFunction<void()> Trace;

		while (Traces.Dequeue(Trace))
		{
			Trace();
		}
	}

	return true;
}


/* FMessageTracer implementation
 *****************************************************************************/

void FMessageTracer::ResetMessages()
{
	MessageInfos.Reset();
	MessageTypes.Reset();

	for (auto& EndpointInfoPair : AddressesToEndpointInfos)
	{
		TSharedPtr<FMessageTracerEndpointInfo>& EndpointInfo = EndpointInfoPair.Value;
		{
			EndpointInfo->ReceivedMessages.Reset();
			EndpointInfo->SentMessages.Reset();
		}
	}

	MessagesResetDelegate.Broadcast();
}


bool FMessageTracer::ShouldBreak(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) const
{
	if (FPlatformProcess::SupportsMultithreading())
	{
		if (Breaking)
		{
			return true;
		}

		for (const auto& Breakpoint : Breakpoints)
		{
			if (Breakpoint->IsEnabled() && Breakpoint->ShouldBreak(Context))
			{
				return true;
			}
		}
	}

	return false;
}
