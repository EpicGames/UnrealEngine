// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Containers/Queue.h"
#include "IMessageContext.h"
#include "IMessageTracer.h"
#include "Containers/Ticker.h"

class IMessageInterceptor;
class IMessageReceiver;
class IMessageSubscription;
class IMessageTracerBreakpoint;

/**
 * Implements a message bus tracers.
 */
class FMessageTracer
	: public IMessageTracer
{
public:

	/** Default constructor. */
	FMessageTracer();

	/** Virtual destructor. */
	virtual ~FMessageTracer();

public:

	/**
	 * Notifies the tracer that a message interceptor has been added to the message bus.
	 *
	 * @param Interceptor The added interceptor.
	 * @param MessageType The type of messages being intercepted.
	 */
	void TraceAddedInterceptor(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType);

	/**
	 * Notifies the tracer that a message recipient has been added to the message bus.
	 *
	 * @param Address The address of the added recipient.
	 * @param Recipient The added recipient.
	 */
	void TraceAddedRecipient(const FMessageAddress& Address, const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Recipient);

	/**
	 * Notifies the tracer that a message subscription has been added to the message bus.
	 *
	 * @param Subscription The added subscription.
	 */
	void TraceAddedSubscription(const TSharedRef<IMessageSubscription, ESPMode::ThreadSafe>& Subscription);

	/**
	 * Notifies the tracer that a message has been dispatched.
	 *
	 * @param Context The context of the dispatched message.
	 * @param Recipient The message recipient.
	 * @param Async Whether the message was dispatched asynchronously.
	 */
	void TraceDispatchedMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Recipient, bool Async);

	/**
	 * Notifies the tracer that a message has been handled.
	 *
	 * @param Context The context of the dispatched message.
	 * @param Recipient The message recipient that handled the message.
	 */
	void TraceHandledMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Recipient);

	/**
	 * Notifies the tracer that a message has been intercepted.
	 *
	 * @param Context The context of the intercepted message.
	 * @param Interceptor The interceptor.
	 */
	void TraceInterceptedMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor);

	/**
	 * Notifies the tracer that a message interceptor has been removed from the message bus.
	 *
	 * @param Interceptor The removed interceptor.
	 * @param MessageType The type of messages that is no longer being intercepted.
	 */
	void TraceRemovedInterceptor(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType);

	/**
	 * Notifies the tracer that a recipient has been removed from the message bus.
	 *
	 * @param Address The address of the removed recipient.
	 */
	void TraceRemovedRecipient(const FMessageAddress& Address);

	/**
	 * Notifies the tracer that a message subscription has been removed from the message bus.
	 *
	 * @param Subscriber The removed subscriber.
	 * @param MessageType The type of messages no longer being subscribed to.
	 */
	void TraceRemovedSubscription(const TSharedRef<IMessageSubscription, ESPMode::ThreadSafe>& Subscription, const FTopLevelAssetPath& MessageType);

	/**
	 * Notifies the tracer that a message has been routed.
	 *
	 * @param Context The context of the routed message.
	 */
	void TraceRoutedMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/**
	 * Notifies the tracer that a message has been sent.
	 *
	 * @param Context The context of the sent message.
	 */
	void TraceSentMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

public:

	//~ IMessageTracer interface

	virtual void Break() override;
	virtual void Continue() override;
	virtual int32 GetEndpoints(TArray<TSharedPtr<FMessageTracerEndpointInfo>>& OutEndpoints) const override;
	virtual int32 GetMessages(TArray<TSharedPtr<FMessageTracerMessageInfo>>& OutMessages) const override;
	virtual int32 GetMessageTypes(TArray<TSharedPtr<FMessageTracerTypeInfo>>& OutTypes) const override;
	virtual bool HasMessages() const override;
	virtual bool IsBreaking() const override;
	virtual bool IsRunning() const override;

	DECLARE_DERIVED_EVENT(FMessageTracer, IMessageTracer::FOnMessageAdded, FOnMessageAdded)
	virtual FOnMessageAdded& OnMessageAdded() override
	{
		return MessagesAddedDelegate;
	}

	DECLARE_DERIVED_EVENT(FMessageTracer, IMessageTracer::FOnMessagesReset, FOnMessagesReset)
	virtual FOnMessagesReset& OnMessagesReset() override
	{
		return MessagesResetDelegate;
	}

	DECLARE_DERIVED_EVENT(FMessageTracer, IMessageTracer::FOnTypeAdded, FOnTypeAdded)
	virtual FOnTypeAdded& OnTypeAdded() override
	{
		return TypeAddedDelegate;
	}

	virtual void Reset() override;
	virtual void Step() override;
	virtual void Stop();
	virtual bool Tick(float DeltaTime) override;

protected:

	/** Resets traced messages. */
	void ResetMessages();

	/**
	 * Checks whether the tracer should break on the given message.
	 *
	 * @param Context The context of the message to consider for breaking.
	 */
	bool ShouldBreak(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) const;

private:

	/** Holds the collection of endpoints for known message addresses. */
	TMap<FMessageAddress, TSharedPtr<FMessageTracerEndpointInfo>> AddressesToEndpointInfos;

	/** Holds a flag indicating whether a breakpoint was hit. */
	bool Breaking;

	/** Holds the collection of breakpoints. */
	TArray<TSharedPtr<IMessageTracerBreakpoint, ESPMode::ThreadSafe>> Breakpoints;

	/** Holds an event signaling that messaging routing can continue. */
	FEvent* ContinueEvent;

	/** The collection of known interceptors. */
	TMap<FGuid, TSharedPtr<FMessageTracerInterceptorInfo>> Interceptors;

	/** Holds the collection of endpoints for known recipient identifiers. */
	TMap<FGuid, TSharedPtr<FMessageTracerEndpointInfo>> RecipientsToEndpointInfos;

	/** Holds the collection of known messages. */
	TMap<TSharedPtr<IMessageContext, ESPMode::ThreadSafe>, TSharedPtr<FMessageTracerMessageInfo>> MessageInfos;

	/** Holds the collection of known message types. */
	TMap<FTopLevelAssetPath, TSharedPtr<FMessageTracerTypeInfo>> MessageTypes;

	/** Holds a flag indicating whether a reset is pending. */
	bool ResetPending;

	/** Holds a flag indicating whether the tracer is running. */
	bool Running;

	/** Handle to the registered TickDelegate. */
	FTSTicker::FDelegateHandle TickDelegateHandle;

	/** Holds the trace actions queue. */
	TQueue<TFunction<void()>, EQueueMode::Mpsc> Traces;

private:

	/** Holds a delegate that is executed when a new message has been added to the collection of known messages. */
	FOnMessageAdded MessagesAddedDelegate;

	/** Holds a delegate that is executed when the message history has been reset. */
	FOnMessagesReset MessagesResetDelegate;

	/** Holds a delegate that is executed when a new type has been added to the collection of known message types. */
	FOnTypeAdded TypeAddedDelegate;
};
