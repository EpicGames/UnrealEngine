// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CoreMisc.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "Bus/MessageDispatchTask.h"
#include "IMessageBus.h"
#include "Bus/MessageBus.h"
#include "Bridge/MessageBridge.h"
#include "IMessagingModule.h"
#include "INetworkMessagingExtension.h"
#include "MessagingTrace.h"


#ifndef PLATFORM_SUPPORTS_MESSAGEBUS
	#define PLATFORM_SUPPORTS_MESSAGEBUS !(WITH_SERVER_CODE && UE_BUILD_SHIPPING)
#endif

DEFINE_LOG_CATEGORY(LogMessaging);

#if MESSAGINGTRACE_ENABLED
UE_TRACE_CHANNEL_DEFINE(MessagingChannel,
	"UDP messaging traffic emitted by UdpMessagingTrace (node discovery, message summary/type info, lifecycle stages); "
	"visualized by the UDP Messaging Insights plugin as Timing View tracks.");
#endif


/**
 * Implements the Messaging module.
 */
class FMessagingModule
	: public FSelfRegisteringExec
	, public IMessagingModule
{
protected:

	//~ FSelfRegisteringExec interface

	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (!FParse::Command(&Cmd, TEXT("MESSAGING")))
		{
			return false;
		}

		if (FParse::Command(&Cmd, TEXT("STATUS")))
		{
			if (DefaultBus.IsValid())
			{
				Ar.Log(TEXT("Default message bus has been initialized."));
			}
			else
			{
				Ar.Log(TEXT("Default message bus has NOT been initialized yet."));
			}
		}
		else
		{
			// show usage
			Ar.Log(TEXT("Usage: MESSAGING <Command>"));
			Ar.Log(TEXT(""));
			Ar.Log(TEXT("Command"));
			Ar.Log(TEXT("    STATUS = Displays the status of the default message bus"));
		}

		return true;
	}

public:

	//~ IMessagingModule interface

	virtual FOnMessageBusStartupOrShutdown& OnMessageBusStartup() override
	{
		return OnMessageBusStartupDelegate;
	}

	virtual FOnMessageBusStartupOrShutdown& OnMessageBusShutdown() override
	{
		return OnMessageBusShutdownDelegate;
	}

	virtual TSharedPtr<IMessageBridge, ESPMode::ThreadSafe> CreateBridge(const FMessageAddress& Address, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& Bus, const TSharedRef<IMessageTransport, ESPMode::ThreadSafe>& Transport) override
	{
		return MakeShared<FMessageBridge, ESPMode::ThreadSafe>(Address, Bus, Transport);
	}

	virtual TSharedPtr<IMessageBus, ESPMode::ThreadSafe> CreateBus(const TSharedPtr<IAuthorizeMessageRecipients>& RecipientAuthorizer) override
	{
		return CreateBus(FGuid::NewGuid().ToString(), RecipientAuthorizer);
	}

	virtual TSharedPtr<IMessageBus, ESPMode::ThreadSafe> CreateBus(FString InName, const TSharedPtr<IAuthorizeMessageRecipients>& RecipientAuthorizer) override
	{
		TSharedRef<IMessageBus, ESPMode::ThreadSafe> Bus = MakeShared<FMessageBus, ESPMode::ThreadSafe>(MoveTemp(InName), RecipientAuthorizer);

		Bus->OnShutdown().AddLambda([this, WeakBus = TWeakPtr<IMessageBus, ESPMode::ThreadSafe>(Bus)]()
		{
			WeakBuses.RemoveSwap(WeakBus);
			OnMessageBusShutdownDelegate.Broadcast(WeakBus);
		});

		WeakBuses.Add(Bus);
		OnMessageBusStartupDelegate.Broadcast(Bus);
		return Bus;
	}

	virtual TSharedPtr<IMessageBus, ESPMode::ThreadSafe> GetDefaultBus() const override
	{
		return DefaultBus;
	}
	
	virtual TArray<TSharedRef<IMessageBus, ESPMode::ThreadSafe>> GetAllBuses() const override
	{
		TArray<TSharedRef<IMessageBus, ESPMode::ThreadSafe>> Buses;
		for (const TWeakPtr<IMessageBus, ESPMode::ThreadSafe>& WeakBus : WeakBuses)
		{
			if (TSharedPtr<IMessageBus, ESPMode::ThreadSafe> Bus = WeakBus.Pin())
			{
				Buses.Add(Bus.ToSharedRef());
			}
		}
		return Buses;
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
#if PLATFORM_SUPPORTS_MESSAGEBUS
		FCoreDelegates::OnPreExit.AddRaw(this, &FMessagingModule::HandleCorePreExit);
		DefaultBus = CreateBus(TEXT("DefaultBus"), nullptr);
#endif	//PLATFORM_SUPPORTS_MESSAGEBUS
	}

	virtual void ShutdownModule() override
	{
		ShutdownDefaultBus();

#if PLATFORM_SUPPORTS_MESSAGEBUS
		FCoreDelegates::OnPreExit.RemoveAll(this);
#endif	//PLATFORM_SUPPORTS_MESSAGEBUS
	}

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

protected:

	void ShutdownDefaultBus()
	{
		if (!DefaultBus.IsValid())
		{
			return;
		}

		TWeakPtr<IMessageBus, ESPMode::ThreadSafe> DefaultBusPtr = DefaultBus;

		DefaultBus->Shutdown();
		DefaultBus.Reset();

		// wait for the bus to shut down
		int32 SleepCount = 0;
		while (DefaultBusPtr.IsValid())
		{
			if (SleepCount > 10)
			{
				check(!"Something is holding on the default message bus");
				break;
			}			
			++SleepCount;
			FPlatformProcess::Sleep(0.1f);
		}

		// validate all other buses were also properly shutdown
		for (TWeakPtr<IMessageBus, ESPMode::ThreadSafe> WeakBus : WeakBuses)
		{
			if (WeakBus.IsValid())
			{
				check(!"Something is holding on a message bus");
				break;
			}
		}
	}

private:

	/** Callback for Core shutdown. */
	void HandleCorePreExit()
	{
		ShutdownDefaultBus();
	}

private:
	/** Holds the message bus. */
	TSharedPtr<IMessageBus, ESPMode::ThreadSafe> DefaultBus;

	/** All buses that were created through this module including the default one. */
	TArray<TWeakPtr<IMessageBus, ESPMode::ThreadSafe>> WeakBuses;

	/** The delegate fired when a message bus instance is started. */
	FOnMessageBusStartupOrShutdown OnMessageBusStartupDelegate;

	/** The delegate fired when a message bus instance is shutdown. */
	FOnMessageBusStartupOrShutdown OnMessageBusShutdownDelegate;
};

FName INetworkMessagingExtension::ModularFeatureName("NetworkMessaging");

IMPLEMENT_MODULE(FMessagingModule, Messaging);
