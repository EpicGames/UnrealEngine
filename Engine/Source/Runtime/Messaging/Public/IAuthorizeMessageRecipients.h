// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"

class FName;
class IMessageInterceptor;
class IMessageReceiver;

struct FMessageAddress;


/**
 * Interface for classes that authorize message subscriptions.
 */
class IAuthorizeMessageRecipients
{
public:

	/**
	 * Authorizes a request to intercept messages of the specified type.
	 *
	 * @param Interceptor The message interceptor to authorize.
	 * @param MessageType The type of messages to intercept.
	 * @return true if the request was authorized, false otherwise.
	 */
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use a version of this function that takes an FTopLevelAssetPath as MessageType.")
	virtual bool AuthorizeInterceptor(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FName& MessageType)
	{
		return AuthorizeInterceptor(Interceptor, UClass::TryConvertShortTypeNameToPathName<UStruct>(MessageType.ToString()));
	}

	/**
	 * Authorizes a request to intercept messages of the specified type.
	 *
	 * @param Interceptor The message interceptor to authorize.
	 * @param MessageType The type of messages to intercept.
	 * @return true if the request was authorized, false otherwise.
	 */
	virtual bool AuthorizeInterceptor(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType) = 0;

	/**
	 * Authorizes a request to register the specified recipient.
	 *
	 * @param Recipient The recipient to register.
	 * @param Address The recipient's address.
	 * @return true if the request was authorized, false otherwise.
	 */
	virtual bool AuthorizeRegistration(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Recipient, const FMessageAddress& Address) = 0;

	/**
	 * Authorizes a request to add a subscription for the specified topic pattern.
	 *
	 * @param Subscriber The subscriber.
	 * @param TopicPattern The message topic pattern to subscribe to.
	 * @return true if the request is authorized, false otherwise.
	 */
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use a version of this function that takes an FTopLevelAssetPath as TopicPattern.")
	virtual bool AuthorizeSubscription(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FName& TopicPattern)
	{
		return AuthorizeSubscription(Subscriber, UClass::TryConvertShortTypeNameToPathName<UStruct>(TopicPattern.ToString()));
	}

	/**
	 * Authorizes a request to add a subscription for the specified topic pattern.
	 *
	 * @param Subscriber The subscriber.
	 * @param TopicPattern The message topic pattern to subscribe to.
	 * @return true if the request is authorized, false otherwise.
	 */
	virtual bool AuthorizeSubscription(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FTopLevelAssetPath& TopicPattern) = 0;

	/**
	 * Authorizes a request to unregister the specified recipient.
	 *
	 * @param Address The address of the recipient to unregister.
	 * @return true if the request was authorized, false otherwise.
	 */
	virtual bool AuthorizeUnregistration(const FMessageAddress& Address) = 0;

	/**
	 * Authorizes a request to remove a subscription for the specified topic pattern.
	 *
	 * @param Subscriber The subscriber.
	 * @param TopicPattern The message topic pattern to unsubscribe from.
	 * @return true if the request is authorized, false otherwise.
	 */
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use a version of this function that takes an FTopLevelAssetPath as TopicPattern.")
	virtual bool AuthorizeUnsubscription(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FName& TopicPattern)
	{
		return AuthorizeUnsubscription(Subscriber, UClass::TryConvertShortTypeNameToPathName<UStruct>(TopicPattern.ToString()));
	}

	/**
	 * Authorizes a request to remove a subscription for the specified topic pattern.
	 *
	 * @param Subscriber The subscriber.
	 * @param TopicPattern The message topic pattern to unsubscribe from.
	 * @return true if the request is authorized, false otherwise.
	 */
	virtual bool AuthorizeUnsubscription(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FTopLevelAssetPath& TopicPattern) = 0;

public:

	/** Virtual destructor. */
	virtual ~IAuthorizeMessageRecipients() = default;
};
