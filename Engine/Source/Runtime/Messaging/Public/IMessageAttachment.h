// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FArchive;


/**
 * Interface for message attachments.
 *
 * Message attachments are optional bulk data objects that can be attached to a message, such as
 * files or memory buffers. Transferring attachments within the same process is very efficient and
 * amounts to copying a pointer to the data. On the other hand, sending message attachments to
 * applications running in other processes or on other computers may - depending on their size -
 * take a considerable amount of time, because they need to be serialized and transferred. For this
 * reason, attachments are transferred separately from messages in such cases.
 *
 * NOTE: Message attachments are not fully implemented yet. In particular, transferring attachments
 * over a network does not work yet. This API may change in the near future, so please use with care.
 *
 * @see IMessageContext
 */
class IMessageAttachment
{
public:

	/**
	 * Creates an archive reader to the data.
	 *
	 * The caller is responsible for deleting the returned object.
	 *
	 * @return An archive reader.
	 */
	virtual FArchive* CreateReader() = 0;

public:

	/** Virtual destructor. */
	virtual ~IMessageAttachment() = default;
};
