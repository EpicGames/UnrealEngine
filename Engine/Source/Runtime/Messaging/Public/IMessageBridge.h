// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"

/*
 * Interface for message bridges.
 *
 * A message bridge connects a message bus with another messaging system. It is really just a regular
 * message endpoint connected to a message bus that translates sent and received messages between the bus
 * and some underlying transport technology. The transport technology is usually implemented in the form of a 
 * Message Transport Plug-in, such as the UdpMessaging plug-in that ships with Unreal Engine.
 *
 * The most common use case for message bridges is to connect two Unreal Engine message buses running in
 * separate processes or en different computers. Another common use case is to connect an Unreal Engine
 * message bus to an entirely different messaging system that is not based on Unreal Engine.
 *
 * Message bridge instances can be created with the IMessagingModule.CreateBridge method.
 */
class IMessageBridge
{
public:

	/**
	 * Disables this bridge.
	 *
	 * A disabled bridge will not receive any subscribed messages until it is enabled again.
	 * Bridges should be created in a disabled state by default and explicitly enabled.
	 *
	 * @see Enable, IsEnabled
	 */
	virtual void Disable() = 0;

	/**
	 * Enables this bridge.
	 *
	 * An activated bridge will receive subscribed messages.
	 * Bridges should be created in a disabled state by default and explicitly enabled.
	 *
	 * @see Disable, IsEnabled
	 */
	virtual void Enable() = 0;

	/**
	 * Checks whether the bridge is currently enabled.
	 *
	 * @return true if the bridge is enabled, false otherwise.
	 * @see Disable, Enable
	 */
	virtual bool IsEnabled() const = 0;

	/**
	 * Provides the endpoint guid for the given address.
	 *
	 * @param MessageAddress to find remote endpoint guid.
	 * @return FGuid which may be invalid.
	 */
	virtual FGuid LookupAddress(const FMessageAddress& InAddress) = 0;

	/**
	 * @param MessageType The type of message to check.
	 * @return True if the bridge is subscribed to the parameter MessageType.
	 */
	virtual bool IsSubscribed(const FTopLevelAssetPath& MessageType) const = 0;

public:

	/** Virtual destructor. */
	virtual ~IMessageBridge() = default;
};
