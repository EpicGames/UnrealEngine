// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Math/Range.h"
#include "Templates/SharedPointer.h"
#include "UObject/TopLevelAssetPath.h"

class FName;
class IMessageAttachment;
class IMessageContext;
class IMessageInterceptor;
class IMessageReceiver;
class IMessageSender;
class IMessageSubscription;
class IMessageTracer;
class IBusListener;
class UScriptStruct;

enum class EMessageBusNotification : uint8;
enum class EMessageScope : uint8;
enum class EMessageFlags : uint32;

struct FDateTime;
struct FMessageAddress;
struct FTimespan;

/** Delegate type for message bus shutdowns. */
DECLARE_MULTICAST_DELEGATE(FOnMessageBusShutdown);


/**
 * Interface for message buses.
 *
 * A message bus is the main logical component to facilitate communication between (possibly distributed) parts of an application
 * using Message Passing as its underlying architectural pattern. It allows registered sender and recipient objects, subsequently
 * referred to as Message Endpoints, to exchange structured data in the form of user defined messages[1].
 *
 * Depending on their usage, messages are classified into a number of types, such as commands, events and documents. In Unreal Engine,
 * all these messages are modeled with regular built-in or user defined UStructs that may either be empty or contain data in the
 * form of FProperty fields[2]. Before being dispatched, messages are internally wrapped into so called Message Context objects
 * (IMessageContext) that contain additional information about a message, such as when it was sent and the sender and recipients.
 *
 * The sending and receiving of messages is not limited to message endpoints within the same thread or process, but may be extended
 * to other applications running on the same computer or on other computers connected to a network with so called Message Transport
 * Plug-ins, such as the UdpMessaging plug-in that ships with Unreal Engine. The main goal of a message bus is to hide the technical
 * details of the underlying transport mechanisms, so that users can focus on implementing their distributed applications rather than
 * worrying about how data gets from one endpoint to another. A message bus makes it look as if all senders and recipients are located
 * within the same process, regardless of whether that is actually the case or not. It is possible to restrict the reach of messages
 * using so called Message Scopes (EMessageScope).
 *
 * All message recipients (objects implementing the see IMessageReceiver interface) must be registered with the message bus through the
 * see IMessageBus.Register method. Before a recipient is destroyed it should unregister itself using the see IMessageBus.Unregister
 * method. Message senders (objects implementing the see IMessageSender interface) do not register with the bus, but instead pass a
 * reference to themselves each time they send a message. The IMessageReceiver and IMessageSender both are very low-level interfaces
 * into the messaging system. Most users will prefer to use instances of the see FMessageEndpoint class instead, which provides a much
 * more convenient way of sending and receiving messages.
 *
 * Message buses in Unreal Engine support the following two common messaging patterns: Request-Reply and Publish-Subscribe. Please note
 * that higher level concepts, such as Remote Procedure Call (RPC) are not part of the messaging system, but we may provide them as
 * separate features in the future.
 *
 * In the Request-Reply pattern a message is sent to one or more particular message recipients using the IMessageBus.Send method.
 * Message recipients implement the see IMessageRecipient interface and are uniquely identified by their addresses (FMessageAddress).
 * After a message is received, the recipients may then reply with another message using the same IMessageBus.Send method. Alternatively,
 * a previously received message may be forwarded to another recipient using the see IMessageBus.Forward method. This pattern is useful
 * when message recipients already know about each other and wish to communicate directly, i.e. to exchange commands or events.
 *
 * In the Publish-Subscribe pattern a message is sent to all message recipients on the bus using the see IMessageBus.Publish method.
 * Only recipients that previously subscribed to the type of the sent message using the IMessageBus.Subscribe method will actually receive
 * the message. All other recipients will not receive the message. After a published message is received, the recipients may respond with
 * another message, either directly to the message sender using IMessageBus.Send method, or by publishing another message using the
 * the IMessageBus.Publish method. This pattern is useful for discovering recipients on the bus and to message recipients that are unknown.
 *
 * Most applications will often use a combination of both Request-Reply and Publish-Subscribe. A typical implementation of distributed
 * applications involves service providers and service consumers (a service can be any useful functionality provided by a system). The
 * service consumers will often publish a special message in order to discover service providers. Service providers are subscribed to these
 * special messages and will respond with another message that contains information about the provider and that is sent back directly to the
 * consumer. The consumer is now aware of the service provider's existence and its message address, and it can then request services by
 * directly sending all subsequent messages directly to it.
 *
 * The dispatching of sent and published messages to the correct recipients is accomplished by the Message Router, which is an internal
 * component of the message bus. The message router maintains an address book of known message endpoints that allows it to determine
 * the destination of messages. If a message cannot be delivered, it is forwarded to a so called Dead Letter Channel, which is a queue
 * of messages that can be inspected for debugging purposes[3].
 *
 * It is possible to intercept messages before they are being routed to recipients by registering so called Message Interceptors (objects
 * implementing the IMessageInterceptor interface) with the message bus. This feature enables advanced use cases that require inspection or
 * manipulation of messages contents, such as message filtering and enriching, splitting and aggregating, re-sequencing or authentication[4].
 *
 * The messaging system also provides a facility for debugging the system itself through the so called Message Tracer (an internal object
 * implementing the see IMessageTracer interface) that can be accessed with the IMessageBus.GetTracer() method. The message tracer is
 * currently used in the Messaging Debugger - a visual debugging tool for the Messaging System in Unreal Frontend and the Unreal Editor.
 *
 * Notes:
 *
 * [1] The messaging implementation in Unreal Engine is more or less closely following the terminology in the Enterprise Integration
 * Patterns book at http://www.eaipatterns.com/eaipatterns.html. A good introduction to messaging concepts can also be found on MSDN
 * at http://msdn.microsoft.com/en-us/library/ff647328.aspx
 *
 * [2] It is currently also required for such structures to implement a specific TStructOpsTypeTraits type trait or otherwise they
 * will not be recognized as legal message types. Search for 'WithMessagingHandling' in the code base to find existing examples. We
 * hope to remove the need for these type traits soon.
 *
 * [3] Dead letter channels are not implemented yet.
 *
 * [4] There are currently no built-in mechanisms for authentication and authorization, but they are on the to-do list.
 *
 * @see FMessageAddress, FMessageEndpoint, IMessageReceiver, IMessageSender
 */
class IMessageBus
{
public:

	/**
	 * Forwards a previously received message.
	 *
	 * Messages can only be forwarded to endpoints within the same process.
	 *
	 * @param Context The context of the message to forward.
	 * @param Recipients The list of message recipients to forward the message to.
	 * @param Delay The deferral time.
	 * @param Forwarder The sender that forwards the message.
	 * @see Publish, Send
	 */
	virtual void Forward(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay, const TSharedRef<IMessageSender, ESPMode::ThreadSafe>& Forwarder) = 0;

	/**
	 * Gets the message bus tracer.
	 *
	 * @return Weak pointer to the tracer object.
	 */
	virtual TSharedRef<IMessageTracer, ESPMode::ThreadSafe> GetTracer() = 0;

	/**
	 * Adds an interceptor for messages of the specified type.
	 *
	 * @param Interceptor The interceptor.
	 * @param MessageType The type of messages to intercept.
	 * @see Unintercept
	 */
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use a version of this function that takes an FTopLevelAssetPath as MessageType.")
	MESSAGING_API virtual void Intercept(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FName& MessageType);

	/**
	 * Adds an interceptor for messages of the specified type.
	 *
	 * @param Interceptor The interceptor.
	 * @param MessageType The type of messages to intercept.
	 * @see Unintercept
	 */
	virtual void Intercept(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType) = 0;

	/**
	 * Sends a message to subscribed recipients.
	 *
	 * The bus takes over ownership of the message's memory.
	 * It must NOT be freed by the caller.
	 *
	 * @param Message The message to publish.
	 * @param TypeInfo The message's type information.
	 * @param Scope The message scope.
	 * @param Annotations An optional message annotations header.
	 * @param Delay The delay after which to send the message.
	 * @param Expiration The time at which the message expires.
	 * @param Publisher The message publisher.
	 * @see Forward, Send
	 */
	virtual void Publish(void* Message, UScriptStruct* TypeInfo, EMessageScope Scope, const TMap<FName, FString>& Annotations, const FTimespan& Delay, const FDateTime& Expiration, const TSharedRef<IMessageSender, ESPMode::ThreadSafe>& Publisher) = 0;

	/**
	 * Registers a message recipient with the message bus.
	 *
	 * @param Address The address of the recipient to register.
	 * @param Recipient The message recipient.
	 * @see Unregister
	 */
	virtual void Register(const FMessageAddress& Address, const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Recipient) = 0;

	/**
	 * Sends a message to multiple recipients.
	 *
	 * The bus takes over ownership of the message's memory.
	 * It must NOT be freed by the caller.
	 *
	 * @param Message The message to send.
	 * @param TypeInfo The message's type information.
	 * @param Flags The message flags.
	 * @param Annotations An optional message annotations header.
	 * @param Attachment The binary data to attach to the message.
	 * @param Recipients The list of message recipients.
	 * @param Delay The delay after which to send the message.
	 * @param Expiration The time at which the message expires.
	 * @param Sender The message sender.
	 * @see Forward, Publish
	 */
	virtual void Send(void* Message, UScriptStruct* TypeInfo, EMessageFlags Flags, const TMap<FName, FString>& Annotations, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const TArray<FMessageAddress>& Recipients, const FTimespan& Delay, const FDateTime& Expiration, const TSharedRef<IMessageSender, ESPMode::ThreadSafe>& Sender) = 0;

	/**
	 * Shuts down the message bus.
	 *
	 * @see OnShutdown
	 */
	virtual void Shutdown() = 0;

	/**
	 * Adds a subscription for published messages of the specified type.
	 *
	 * Subscriptions allow message consumers to receive published messages from the message bus.
	 * The returned interface can be used to query the subscription's details and its enabled state.
	 *
	 * @param Subscriber The subscriber wishing to receive the messages.
	 * @param MessageType The type of messages to subscribe to (NAME_All = subscribe to all message types).
	 * @param ScopeRange The range of message scopes to include in the subscription.
	 * @return The added subscription, or nullptr if the subscription failed.
	 * @see Unsubscribe
	 */
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use a version of this function that takes an FTopLevelAssetPath as MessageType.")
	MESSAGING_API virtual TSharedPtr<IMessageSubscription, ESPMode::ThreadSafe> Subscribe(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FName& MessageType, const TRange<EMessageScope>& ScopeRange);

	/**
	 * Adds a subscription for published messages of the specified type.
	 *
	 * Subscriptions allow message consumers to receive published messages from the message bus.
	 * The returned interface can be used to query the subscription's details and its enabled state.
	 *
	 * @param Subscriber The subscriber wishing to receive the messages.
	 * @param MessageType The type of messages to subscribe to (IMessageBus::PATHNAME_All = subscribe to all message types).
	 * @param ScopeRange The range of message scopes to include in the subscription.
	 * @return The added subscription, or nullptr if the subscription failed.
	 * @see Unsubscribe
	 */
	virtual TSharedPtr<IMessageSubscription, ESPMode::ThreadSafe> Subscribe(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FTopLevelAssetPath& MessageType, const TRange<EMessageScope>& ScopeRange) = 0;

	/**
	 * Removes an interceptor for messages of the specified type.
	 *
	 * @param Interceptor The interceptor to remove.
	 * @param MessageType The type of messages to stop intercepting.
	 * @see Intercept
	 */
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use a version of this function that takes an FTopLevelAssetPath as MessageType.")
	MESSAGING_API virtual void Unintercept(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FName& MessageType);

	/**
	 * Removes an interceptor for messages of the specified type.
	 *
	 * @param Interceptor The interceptor to remove.
	 * @param MessageType The type of messages to stop intercepting.
	 * @see Intercept
	 */
	virtual void Unintercept(const TSharedRef<IMessageInterceptor, ESPMode::ThreadSafe>& Interceptor, const FTopLevelAssetPath& MessageType) = 0;

	/**
	 * Unregisters a message recipient from the message bus.
	 *
	 * @param Address The address of the recipient to unregister.
	 * @see Register
	 */
	virtual void Unregister(const FMessageAddress& Address) = 0;

	/**
	 * Cancels the specified message subscription.
	 *
	 * @param Subscriber The subscriber wishing to stop receiving the messages.
	 * @param MessageType The type of messages to unsubscribe from (NAME_All = all types).
	 * @see Subscribe
	 */
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use a version of this function that takes an FTopLevelAssetPath as MessageType.")
	MESSAGING_API virtual void Unsubscribe(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FName& MessageType);

	/**
	 * Cancels the specified message subscription.
	 *
	 * @param Subscriber The subscriber wishing to stop receiving the messages.
	 * @param MessageType The type of messages to unsubscribe from (IMessageBus::PATHNAME_All = all types).
	 * @see Subscribe
	 */
	virtual void Unsubscribe(const TSharedRef<IMessageReceiver, ESPMode::ThreadSafe>& Subscriber, const FTopLevelAssetPath& MessageType) = 0;

	/**
	 * Add a listener to the bus notifications
	 * @param Listener The listener to as to the registration notifications
	 */
	virtual void AddNotificationListener(const TSharedRef<IBusListener, ESPMode::ThreadSafe>& Listener) = 0;

	/**
	 * Remove a listener to the bus notifications
	 * @param Listener The listener to remove from the registration notifications
	 */
	virtual void RemoveNotificationListener(const TSharedRef<IBusListener, ESPMode::ThreadSafe>& Listener) = 0;

	/**
	 * @return The name of this message bus. 
	 */
	virtual const FString& GetName() const = 0;

public:

	/**
	 * Returns a delegate that is executed when the message bus is shutting down.
	 *
	 * @return The delegate.
	 * @see Shutdown
	 */
	virtual FOnMessageBusShutdown& OnShutdown() = 0;

	MESSAGING_API static const FTopLevelAssetPath PATHNAME_All;

public:

	/** Virtual destructor. */
	virtual ~IMessageBus() = default;
};
