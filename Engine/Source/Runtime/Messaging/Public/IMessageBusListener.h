// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"

struct FMessageAddress;

enum class EMessageBusNotification : uint8
{
	Registered = 0,
	Unregistered
};

/**
 * Interface for message bus listener.
 *
 * Classes that implement this interface are able to receive notifications from a message bus. A bus listener will receive
 * a call to its appropriate IBusListener.Notify* method for dispatched notifications it is listening to.
 *
 * This interface provides a rather low-level mechanism for listening to notification. Instead of implementing it, Most users will
 * want to use an instance of see FMessageEndpoint, which provides a much more convenient way of listening to bus notifications.
 *
 * @see FMessageEndpoint, IMessageBus
 */
class IBusListener
{
public:
	virtual ~IBusListener() = default;

	/**
	 * Gets the name of the thread on which to receive notifications.
	 *
	 * If the listener's Notify[*] methods are thread-safe, return ThreadAny for best performance.
	 *
	 * @return Name of the listener thread.
	 */
	virtual ENamedThreads::Type GetListenerThread() const = 0;

	/**
	 * Notify a registration event from the bus
	 * This is called when a receiver is registered or unregistered from the bus.
	 * 
	 * @param Address The address of the recipient that just un/registered from the bus.
	 * @param Notification The even type, either `Registered` or `Unregistered`
	 */
	virtual void NotifyRegistration(const FMessageAddress& Address, EMessageBusNotification Notification) = 0;
};
