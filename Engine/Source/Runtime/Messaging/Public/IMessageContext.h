// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Misc/Crc.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IMessageAttachment;

struct FDateTime;


/**
 * Structure for message endpoint addresses.
 */
struct FMessageAddress
{
public:

	/** Default constructor (no initialization). */
	FMessageAddress() { }

public:

	/**
	 * Compares two message addresses for equality.
	 *
	 * @param X The first address to compare.
	 * @param Y The second address to compare.
	 * @return true if the addresses are equal, false otherwise.
	 */
	friend bool operator==(const FMessageAddress& X, const FMessageAddress& Y)
	{
		return (X.UniqueId == Y.UniqueId);
	}

	/**
	 * Compares two message addresses for inequality.
	 *
	 * @param X The first address to compare.
	 * @param Y The second address to compare.
	 * @return true if the addresses are not equal, false otherwise.
	 */
	friend bool operator!=(const FMessageAddress& X, const FMessageAddress& Y)
	{
		return (X.UniqueId != Y.UniqueId);
	}

	/**
	 * Serializes a message address from or into an archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param G The address to serialize.
	 */
	friend FArchive& operator<<(FArchive& Ar, FMessageAddress& A)
	{
		return Ar << A.UniqueId;
	}

public:

	/**
	 * Invalidates the GUID.
	 *
	 * @see IsValid
	 */
	void Invalidate()
	{
		UniqueId.Invalidate();
	}

	/**
	 * Checks whether this message address is valid or not.
	 *
	 * @return true if valid, false otherwise.
	 * @see Invalidate
	 */
	bool IsValid() const
	{
		return UniqueId.IsValid();
	}

	/**
	 * Converts this GUID to its string representation.
	 *
	 * @return The string representation.
	 * @see Parse
	 */
	FString ToString() const
	{
		return UniqueId.ToString();
	}

public:

	/**
	 * Calculates the hash for a message address.
	 *
	 * @param Address The address to calculate the hash for.
	 * @return The hash.
	 */
	friend uint32 GetTypeHash(const FMessageAddress& Address)
	{
		return FCrc::MemCrc_DEPRECATED(&Address.UniqueId, sizeof(FGuid));
	}

public:

	/**
	 * Returns a new message address.
	 *
	 * @return A new address.
	 */
	static FMessageAddress NewAddress()
	{
		FMessageAddress Result;
		Result.UniqueId = FGuid::NewGuid();

		return Result;
	}

	/**
	 * Converts a string to a message address.
	 *
	 * @param String The string to convert.
	 * @param OutAddress Will contain the parsed address.
	 * @return true if the string was converted successfully, false otherwise.
	 * @see ToString
	 */
	static bool Parse(const FString& String, FMessageAddress& OutAddress)
	{
		return FGuid::Parse(String, OutAddress.UniqueId);
	}

private:

	/** Holds a unique identifier. */
	FGuid UniqueId;
};


/**
 * Enumerates scopes for published messages.
 *
 * The scope determines to which endpoints a published message will be delivered.
 * By default, messages will be published to everyone on the local network (Subnet),
 * but it is often useful to restrict the group of recipients to more local scopes,
 * or to widen it to a larger audience outside the local network.
 *
 * For example, if a message is to be handled only by subscribers in the same application,
 * the scope should be set to Process. If messages need to be published between
 * different networks (i.e. between LAN and WLAN), it should be set to Network instead.
 * 
 * Scopes only apply to published messages. Messages that are being sent to specific
 * recipients will always be delivered, regardless of the endpoint locations.
 */
enum class EMessageScope : uint8
{
	/** Deliver to subscribers in the same thread. */
	Thread,

	/** Deliver to subscribers in the same process. */
	Process,

	/** Deliver to subscribers on the network. */
	Network,

	/**
	 * Deliver to all subscribers.
	 *
	 * Note: This must be the last value in this enumeration.
	 */
	All
};

enum class EMessageFlags : uint32
{
	/** No special flags */
	None = 0,
		
	/** Guarantee that this message is delivered */
	Reliable = 1 << 0,
		
	/** Trace this object through the transport on message bus. Only supported if the underlying transport handles the flag. */
	Traced = 1 << 1
};
ENUM_CLASS_FLAGS(EMessageFlags);


/** Type definition for message scope ranges. */
typedef TRange<EMessageScope> FMessageScopeRange;

/** Type definition for message scope range bounds. */
typedef TRangeBound<EMessageScope> FMessageScopeRangeBound;


/**
 * Interface for message contexts.
 *
 * Messages are delivered inside message contexts, which store the message itself plus additional data
 * associated with the message. Recipients of a message are usually interested in data that describes the
 * message, such as its origin or when it expires. They may also be interested in optional out-of-band
 * binary data that is attached to the message.
 *
 * The sender's address (IMessageContext.GetSender) is often needed to send a reply message to a message sender, i.e.
 * in response to a published message. The message attachment (IMessageContext.GetAttachment) is an optional bundle
 * of binary bulk data that is transferred independently from the message itself and allows for transferring
 * larger amounts of data that would otherwise clog up the messaging system.
 *
 * In case a message was forwarded by another endpoint, the context of the original sender can be accessed
 * using the IMessageContext.GetOriginalContext method.
 *
 * @see IMessageAttachment
 */
class IMessageContext
{
public:

	/**
	 * Gets the optional message annotations.
	 *
	 * @return Message header collection.
	 */
	virtual const TMap<FName, FString>& GetAnnotations() const = 0;

	/**
	 * Gets the message attachment, if present.
	 *
	 * @return A pointer to the message attachment, or nullptr if no attachment is present.
	 */
	virtual TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> GetAttachment() const = 0;

	/**
	 * Gets the date and time at which the message expires.
	 *
	 * @return Expiration time.
	 */
	virtual const FDateTime& GetExpiration() const = 0;

	/**
	 * Gets the message data.
	 *
	 * @return A pointer to the message data.
	 * @see GetAttachment, GetMessageType, GetMessageTypeInfo
	 */
	virtual const void* GetMessage() const = 0;

	/**
	 * Gets the message's type information.
	 *
	 * @return Message type information.
	 * @see GetMessage, GetMessageType
	 */
	virtual const TWeakObjectPtr<UScriptStruct>& GetMessageTypeInfo() const = 0;

	/**
	 * Returns the original message context in case the message was forwarded.
	 *
	 * @return The original message context, or nullptr if the message wasn't forwarded.
	 */
	virtual TSharedPtr<IMessageContext, ESPMode::ThreadSafe> GetOriginalContext() const = 0;

	/**
	 * Gets the list of message recipients.
	 *
	 * @return Message recipients.
	 * @see GetSender
	 */
	virtual const TArray<FMessageAddress>& GetRecipients() const = 0;

	/**
	 * Gets the scope to which the message was sent.
	 *
	 * @return The message scope.
	 */
	virtual EMessageScope GetScope() const = 0;

	/**
	* Gets the scope to which the message was sent.
	*
	* @return The message scope.
	*/
	virtual EMessageFlags GetFlags() const = 0;

	/**
	 * Gets the sender's address.
	 *
	 * @return Sender address.
	 * @see GetRecipients, GetSenderThread
	 */
	virtual const FMessageAddress& GetSender() const = 0;

	/**
	 * Gets the forwarder's address.
	 * @note Identical to GetSender if the message wasn't forwarded.
	 * @return Forwarder address.
	 * @see GetSender, GetRecipients, GetSenderThread
	 */
	virtual const FMessageAddress& GetForwarder() const = 0;
	
	/**
	 * Gets the name of the thread from which the message was sent.
	 *
	 * @return Sender threat name.
	 * @see GetSender
	 */
	virtual ENamedThreads::Type GetSenderThread() const = 0;

	/**
	 * Gets the time at which the message was forwarded.
	 *
	 * @return Time forwarded.
	 * @see GetTimeSent, IsForwarded
	 */
	virtual const FDateTime& GetTimeForwarded() const = 0;

	/**
	 * Gets the time at which the message was sent.
	 *
	 * @return Time sent.
	 * @see GetTimeForwarded
	 */
	virtual const FDateTime& GetTimeSent() const = 0;

public:

	/**
	 * Gets the name of the message type.
	 *
	 * @return Message type name.
	 * @see GetMessage, GetMessageTypeInfo
	 */
	UE_DEPRECATED(5.1, "Type names are now represented by path names. Please use GetMessageTypePathName.")
	FName GetMessageType() const
	{
		if (IsValid())
		{
			UStruct* MessageTypeInfoPtr = GetMessageTypeInfo().Get();
			return MessageTypeInfoPtr->GetFName();
		}
		
		return NAME_None;
	}

	/**
	 * Gets the path name of the message type.
	 *
	 * @return Message type path name.
	 * @see GetMessage, GetMessageTypeInfo
	 */
	FTopLevelAssetPath GetMessageTypePathName() const
	{
		if (IsValid())
		{
			UStruct* MessageTypeInfoPtr = GetMessageTypeInfo().Get();
			return MessageTypeInfoPtr->GetStructPathName();
		}

		return FTopLevelAssetPath();
	}

	/**
	 * Checks whether this is a forwarded message.
	 *
	 * @return true if the message was forwarded, false otherwise.
	 * @see GetOriginalContext, GetTimeForwarded, IsValid
	 */
	bool IsForwarded() const
	{
		return GetOriginalContext().IsValid();
	}

	/**
	 * Checks whether this context is valid.
	 *
	 * @return true if the context is valid, false otherwise.
	 * @see IsForwarded
	 */
	bool IsValid() const
	{
		return ((GetMessage() != nullptr) && GetMessageTypeInfo().IsValid(false, true));
	}

public:

	/** Virtual destructor. */
	virtual ~IMessageContext() = default;
};
