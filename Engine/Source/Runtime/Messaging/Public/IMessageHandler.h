// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IMessageContext;


/**
 * Interface for message handlers.
 */
class IMessageHandler
{
public:

	/**
	 * Handles the specified message.
	 *
	 * @param Context The context of the message to handle.
	 */
	virtual void HandleMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMessageHandler() = default;
};
