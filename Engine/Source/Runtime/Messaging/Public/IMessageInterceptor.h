// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IMessageContext;
struct FGuid;


/**
 * Interface for message interceptors.
 */
class IMessageInterceptor
{
public:

	/**
	 * Gets the interceptor's name (for debugging purposes).
	 *
	 * @return The debug name.
	 * @see GetInterceptorId
	 */
	virtual FName GetDebugName() const = 0;

	/**
	 * Gets the interceptor's unique identifier (for debugging purposes).
	 *
	 * @return The interceptor's identifier.
	 * @see GetDebugName
	 */
	virtual const FGuid& GetInterceptorId() const = 0;

	/**
	 * Intercepts a message before it is being passed to the message router.
	 *
	 * @param Context The context of the message to intercept.
	 * @return true if the message was intercepted and should not be routed, false otherwise.
	 */
	virtual bool InterceptMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMessageInterceptor() = default;
};
