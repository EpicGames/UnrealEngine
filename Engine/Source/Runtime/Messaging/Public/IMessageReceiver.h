// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IMessageContext;
struct FGuid;


/**
 * Interface for message recipients.
 *
 * Classes that implement this interface are able to receive messages from a message bus. A message recipient will receive
 * a call to its IMessageReceiver.ReceiveMessage method for each message that was sent directly to it (via IMessageBus.Send)
 * and for each published message (via IMessageBus.Publish) that it subscribed to (via IMessageBus.Subscribe).
 *
 * This interface provides a rather low-level mechanism for receiving messages. Instead of implementing it, Most users will
 * want to use an instance of see FMessageEndpoint, which provides a much more convenient way of sending and receiving messages.
 *
 * @see FMessageEndpoint, IMessageBus, IMessageSender
 */
class IMessageReceiver
{
public:

	/**
	 * Gets the recipient's name (for debugging purposes).
	 *
	 * @return The debug name.
	 * @see GetRecipientId, GetRecipientThread
	 */
	virtual FName GetDebugName() const = 0;

	/**
	 * Gets the recipient's unique identifier (for debugging purposes).
	 *
	 * @return The recipient's identifier.
	 * @see GetDebugName, GetRecipientThread
	 */
	virtual const FGuid& GetRecipientId() const = 0;

	/**
	 * Gets the name of the thread on which to receive messages.
	 *
	 * If the recipient's ReceiveMessage() is thread-safe, return ThreadAny for best performance.
	 *
	 * @return Name of the receiving thread.
	 * @see GetDebugName, GetRecipientId
	 */
	virtual ENamedThreads::Type GetRecipientThread() const = 0;

	/**
	 * Checks whether this recipient represents a local endpoint.
	 *
	 * Local recipients are located in the current thread or process. Recipients located in
	 * other processes on the same machine or on remote machines are considered remote.
	 *
	 * @return true if this recipient is local, false otherwise.
	 * @see IsRemote
	 */
	virtual bool IsLocal() const = 0;

	/**
	 * Handles the given message.
	 *
	 * @param Context Will hold the context of the received message.
	 */
	virtual void ReceiveMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) = 0;

public:

	/**
	 * Checks whether this recipient represents a remote endpoint.
	 *
	 * Local recipients are located in the current thread or process. Recipients located in
	 * other processes on the same machine or on remote machines are considered remote.
	 *
	 * @return true if this recipient is remote, false otherwise.
	 * @see IsLocal
	 */
	bool IsRemote() const
	{
		return !IsLocal();
	}

public:

	/** Virtual destructor. */
	virtual ~IMessageReceiver() = default;
};
