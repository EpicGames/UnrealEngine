// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"
#include "Templates/SharedPointer.h"

class FString;


/**
 * Interface for message senders.
 *
 * Classes that implement this interface are able to send messages on a message bus. Each message sender must be uniquely
 * identifiable with a message address that is returned by the IMessageSender.GetSenderAddress method. It is recommended
 * that implementors of this interface generate a GUID for each instance, which must not change throughout the lifetime of
 * the instance.
 *
 * The sending of messages is accomplished with the IMessageBus.Forward, IMessageBus.Publish and IMessageBus.Send methods.
 * In case an error occurs during the sending of a message, the IMessageSender.NotifyMessageError method will be called.
 *
 * This interface provides a rather low-level mechanism for receiving messages. Instead of implementing it, Most users will
 * want to use an instance of FMessageEndpoint, which provides a much more convenient way of sending and receiving messages.
 *
 * @see FMessageEndpoint, IMessageBus, IMessageReceiver
 */
class IMessageSender
{
public:

	/**
	 * Gets the sender's address.
	 *
	 * @return The message address.
	 */
	virtual FMessageAddress GetSenderAddress() = 0;

	/**
	 * Notifies the sender of errors.
	 *
	 * @param Context The context of the message that generated the error.
	 * @param Error The error string.
	 */
	virtual void NotifyMessageError(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FString& Error) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMessageSender() = default;
};
