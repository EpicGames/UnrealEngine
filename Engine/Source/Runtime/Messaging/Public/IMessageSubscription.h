// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"

class IMessageReceiver;
enum class EMessageScope : uint8;

/**
 * Interface for message subscriptions.
 *
 * This interface provides access to a message subscription that was previously created with the
 * IMessageBus.Subscribe method. It can be used to query the subscription's details using the
 * various getters and modify its enabled state using the IMessageSubscription.Enable and
 * IMessageSubscription.Disable methods.
 *
 * A subscription that is disabled will cause a message endpoint to stop receiving messages for
 * the subscribed message type. However, the subscription is merely disabled and not removed.
 */
class IMessageSubscription
{
public:

	/**
	 * Disables the subscription.
	 *
	 * @see Enable, IsEnabled
	 */
	virtual void Disable() = 0;

	/**
	 * Enables the subscription.
	 *
	 * @see Disable, IsEnabled
	 */
	virtual void Enable() = 0;

	/**
	 * Gets the type of subscribed messages.
	 *
	 * @return Message type.
	 * @see GetScopeRange, GetSubscriber
	 */
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use GetMessageTypePathName.")
	inline FName GetMessageType()
	{
		return GetMessageTypePathName().GetAssetName();
	}

	/**
	 * Gets the type of subscribed messages.
	 *
	 * @return Message type.
	 * @see GetScopeRange, GetSubscriber
	 */
	virtual FTopLevelAssetPath GetMessageTypePathName() = 0;

	/**
	 * Gets the range of subscribed message scopes.
	 *
	 * @return Message scope range.
	 * @see GetMessageType, GetSubscriber
	 */
	virtual const TRange<EMessageScope>& GetScopeRange() = 0;

	/**
	 * Gets the subscriber.
	 *
	 * @return The subscriber.
	 * @see GetMessageType, GetScopeRange
	 */
	virtual const TWeakPtr<IMessageReceiver, ESPMode::ThreadSafe>& GetSubscriber() = 0;

	/**
	 * Checks whether the subscription is enabled.
	 *
	 * @return true if the subscription is enabled, false otherwise.
	 * @see Enable, Disable
	 */
	virtual bool IsEnabled() = 0;

public:

	/** Virtual destructor. */
	virtual ~IMessageSubscription() = default;
};
