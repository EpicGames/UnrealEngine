// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

#include "IMessageContext.h"

struct FMessageTracerEndpointInfo;
struct FMessageTracerMessageInfo;
struct FMessageTracerTypeInfo;


/**
 * Enumerates tracer breakpoint states.
 */
enum class EMessageTracerBreakpointState
{
	/** The breakpoint is disabled. */
	Disabled,

	/** The breakpoint is enabled. */
	Enabled,

	/** The breakpoint is enabled for incoming messages. */
	EnabledIn,

	/** The breakpoint is enabled for outgoing messages. */
	EnabledOut,
};


/**
 * Enumerates message dispatch types.
 */
enum class EMessageTracerDispatchTypes
{
	/** The message is being dispatched directly. */
	Direct,

	/** The message hasn't been dispatched yet. */
	Pending,

	/** The message is being dispatched using the task graph system. */
	TaskGraph
};


/**
 * Structure for tracer breakpoints.
 */
struct FMessageTracerBreakpoint
{
	/** Recipient address to break on. */
	FMessageAddress BreakOnRecipient;

	/** Sender address to break on. */
	FMessageAddress BreakOnSender;

	/** How many times the breakpoint was hit. */
	int64 HitCount;

	/** The breakpoint's enabled state. */
	EMessageTracerBreakpointState State;
};


/**
 * Structure for message dispatch states.
 */
struct FMessageTracerDispatchState
{
	/** Holds the dispatch latency (in seconds). */
	double DispatchLatency;

	/** Holds the message's dispatch type for the specified endpoint. */
	EMessageTracerDispatchTypes DispatchType;

	/** Holds the endpoint to which the message was or is being dispatched. */
	TSharedPtr<FMessageTracerEndpointInfo> EndpointInfo;

	/** The thread on which the endpoint receives messages. */
	ENamedThreads::Type RecipientThread;

	/** Holds the time at which the message was dispatched. */
	double TimeDispatched;

	/** Holds the time at which the message was actually handled (0.0 = not handled yet). */
	double TimeHandled;
};


/**
 * Structure for a recipient's address information.
 */
struct FMessageTracerAddressInfo
{
	/** Holds a recipient's message address. */
	FMessageAddress Address;

	/** Holds the time at which this address was registered. */
	double TimeRegistered;

	/** Holds the time at which this address was unregistered. */
	double TimeUnregistered;
};


/**
 * Structure for message endpoint debug information.
 */
struct FMessageTracerEndpointInfo
{
	/** Holds the recipient's address information. */
	TMap<FMessageAddress, TSharedPtr<FMessageTracerAddressInfo>> AddressInfos;

	/** Holds the recipient's human readable name. */
	FName Name;

	/** Holds the list of messages received by this recipient. */
	TArray<TSharedPtr<FMessageTracerMessageInfo>> ReceivedMessages;

	/** Holds a flag indicating whether this is a remote recipient. */
	bool Remote;

	/** Holds the list of messages sent by this recipient. */
	TArray<TSharedPtr<FMessageTracerMessageInfo>> SentMessages;
};


/**
 * Structure for message interceptor debug information.
 */
struct FMessageTracerInterceptorInfo
{
	/** Holds the interceptor's human readable name. */
	FName Name;

	/** Holds the list of messages intercepted by this interceptor. */
	TArray<TSharedPtr<FMessageTracerMessageInfo>> InterceptedMessages;

	/** Holds the time at which this interceptor was registered. */
	double TimeRegistered;

	/** Holds the time at which this interceptor was unregistered. */
	double TimeUnregistered;
};


/**
 * Structure for message debug information.
 */
struct FMessageTracerMessageInfo
{
	/** Holds a pointer to the message context. */
	TSharedPtr<IMessageContext, ESPMode::ThreadSafe> Context;

	/** Holds the message's dispatch states per endpoint. */
	TMap<TSharedPtr<FMessageTracerEndpointInfo>, TSharedPtr<FMessageTracerDispatchState>> DispatchStates;

	/** Whether the message was intercepted. */
	bool Intercepted;

	/** Pointer to the sender's endpoint information. */
	TSharedPtr<FMessageTracerEndpointInfo> SenderInfo;

	/** Holds the time at which the message was routed (0.0 = pending). */
	double TimeRouted;

	/** Holds the time at which the message was sent. */
	double TimeSent;

	/** Pointer to the message's type information. */
	TSharedPtr<FMessageTracerTypeInfo> TypeInfo;
};


/**
 * Structure for message type debug information.
 */
struct FMessageTracerTypeInfo
{
	/** Holds the collection of messages of this type. */
	TArray<TSharedPtr<FMessageTracerMessageInfo>> Messages;

	/** Holds a name of the message type. */
	UE_DEPRECATED(5.1, "Types names are now represented by path names. Please use TypePathName.")
	FName TypeName;

	/** Holds a path name of the message type. */
	FTopLevelAssetPath TypePathName;
};


/**
 * Interface for message tracers.
 *
 * This interface provides access to the message tracer that is built into a message bus. Message tracers are
 * able to monitor and record all activities on a message bus. This includes the sending and receiving of messages,
 * the registering and unregistering of message endpoints and interceptors and adding and removal of subscriptions.
 */
class IMessageTracer
{
public:

	/**
	 * Breaks message routing.
	 *
	 * @see Continue, IsBreaking, Step
	 */
	virtual void Break() = 0;

	/**
	 * Starts the tracer or continues message routing from the current breakpoint.
	 *
	 * @see Break, Step
	 */
	virtual void Continue() = 0;

	/**
	 * Checks whether the tracer is currently at a breakpoint.
	 *
	 * @return true if at breakpoint, false otherwise.
	 * @see Break, IsRunning
	 */
	virtual bool IsBreaking() const = 0;

	/**
	 * Checks whether the tracer is currently running.
	 *
	 * @return true if the tracer is running, false otherwise.
	 * @see Continue, IsBreaking, Stop
	 */
	virtual bool IsRunning() const = 0;

	/** Resets the tracer. */
	virtual void Reset() = 0;

	/**
	 * Steps the tracer to the next message.
	 *
	 * @see Break, Continue, Stop
	 */
	virtual void Step() = 0;

	/**
	 * Stops the tracer.
	 *
	 * @see Continue, IsRunning, Step
	 */
	virtual void Stop() = 0;

	/**
	 * Ticks the tracer.
	 *
	 * @param DeltaTime The time in seconds since the last tick.
	 * @return true if any events were processed.
	 */
	virtual bool Tick(float DeltaTime) = 0;

public:

	/**
	 * Gets the list of known message endpoints.
	 *
	 * @param OutEndpoints Will contain the list of endpoints.
	 * @return The number of endpoints returned.
	 */
	virtual int32 GetEndpoints(TArray<TSharedPtr<FMessageTracerEndpointInfo>>& OutEndpoints) const = 0;

	/**
	 * Gets the collection of known messages.
	 *
	 * @return The messages.
	 */
	virtual int32 GetMessages(TArray<TSharedPtr<FMessageTracerMessageInfo>>& OutMessages) const = 0;

	/**
	 * Gets the list of known message types filtered by name.
	 *
	 * @param NameFilter The name substring to filter with.
	 * @param OutTypes Will contain the list of message types.
	 * @return The number of message types returned.
	 */
	virtual int32 GetMessageTypes(TArray<TSharedPtr<FMessageTracerTypeInfo>>& OutTypes) const = 0;

	/**
	 * Checks whether there are any messages in the history.
	 *
	 * @return true if there are messages, false otherwise.
	 */
	virtual bool HasMessages() const = 0;

public:

	typedef TSharedRef<FMessageTracerMessageInfo> FMessageTracerMessageInfoRef;
	typedef TSharedRef<FMessageTracerTypeInfo> FMessageTracerTypeInfoRef;

	/**
	 * A delegate that is executed when the collection of known messages has changed.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(IMessageTracer, FOnMessageAdded, FMessageTracerMessageInfoRef)
	virtual FOnMessageAdded& OnMessageAdded() = 0;

	/**
	 * A delegate that is executed when the message history has been reset.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT(IMessageTracer, FOnMessagesReset)
	virtual FOnMessagesReset& OnMessagesReset() = 0;

	/**
	 * A delegate that is executed when the collection of known messages types has changed.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(IMessageTracer, FOnTypeAdded, FMessageTracerTypeInfoRef)
	virtual FOnTypeAdded& OnTypeAdded() = 0;

protected:

	/** Hidden destructor. The life time of a message tracer is managed by the message bus. */
	virtual ~IMessageTracer() = default;
};
