// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IMessageContext;


/**
 * Interface for message tracer breakpoints.
 *
 * @see IMessageTracer
 */
class IMessageTracerBreakpoint
{
public:

	/**
	 * Checks whether this breakpoint is enabled.
	 *
	 * @return true if the breakpoint is enabled, false otherwise.
	 */
	virtual bool IsEnabled() const = 0;

	/**
	 * Checks whether the tracer should break on the given message.
	 *
	 * @param Context The context of the message to break on.
	 * @return true if the tracer should break, false otherwise.
	 */
	virtual bool ShouldBreak(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) const = 0;

protected:

	/** Hidden destructor. */
	virtual ~IMessageTracerBreakpoint() = default;
};
