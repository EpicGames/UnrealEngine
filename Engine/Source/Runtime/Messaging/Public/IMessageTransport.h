// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IMessageContext;
class IMessageTransportHandler;


/**
 * Interface for message transport technologies.
 *
 * Licensees can implement this interface to add support for custom message transport
 * technologies that are not supported out of the box, i.e. custom network protocols or APIs.
 */
class IMessageTransport
{
public:

	/**
	 * Gets the name of this transport (for debugging purposes).
	 *
	 * @return The debug name.
	 */
	virtual FName GetDebugName() const = 0;

	/**
	 * Starts up the message transport.
	 *
	 * @param Handler The handler of inbound transport messages and events.
	 * @return Whether the transport was started successfully.
	 * @see StopTransport
	 */
	virtual bool StartTransport(IMessageTransportHandler& Handler) = 0;

	/**
	 * Shuts down the message transport.
	 *
	 * @see StartTransport
	 */
	virtual void StopTransport() = 0;

	/**
	 * Transports the given message data to the specified network nodes.
	 *
	 * @param Context The context of the message to transport.
	 * @param Recipients The transport nodes to send the message to.
	 * @return true if the message is being transported, false otherwise.
	 */
	virtual bool TransportMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TArray<FGuid>& Recipients) = 0;

protected:

	/** Virtual destructor. */
	virtual ~IMessageTransport() = default;
};
