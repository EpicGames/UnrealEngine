// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IMessageContext;
struct FGuid;


/**
 * Interface for message transport event handlers.
 *
 * This interface is typically implemented by message bridges and other classes
 * that manage a message transport and wish to receive messages from it. This
 * interface also notifies implementors of discovered and lost transport nodes.
 *
 * A transport node is the internal representation of a remote message endpoint
 * that may exist in a different process or on a different computer. The actual
 * implementation of these nodes depends on the transport layer. It may be a
 * network socket connection, a named pipe, or some other communication technology.
 *
 * Each transport node gets a globally unique identifier that can be used by a
 * message bridge to translate local message addresses to remote message endpoints.
 * When a message endpoint on the message bus sends a message to a specific message
 * address that represents a remote endpoint, the message bridge maps the address
 * to a transport identifier, which is then mapped again to the corresponding
 * connection in the transport layer.
 *
 * @see IMessageBridge, IMessageBus, IMessageContext, IMessageTransport
 */
class IMessageTransportHandler
{
public:

	/**
	 * Called by message transports when a transport node has been discovered.
	 *
	 * @param NodeId The identifier of the discovered transport node.
	 * @see ForgetTransportNode
	 */
	virtual void DiscoverTransportNode(const FGuid& NodeId) = 0;

	/**
	 * Called by message transports when a transport node has been lost.
	 *
	 * @param NodeId The identifier of the lost transport node.
	 * @see DiscoverTransportNode
	 */
	virtual void ForgetTransportNode(const FGuid& NodeId) = 0;

	/**
	 * Called by message transports when a message was received.
	 *
	 * @param Context The context of the received message.
	 * @param NodeId The identifier of the transport node that received the message.
	 */
	virtual void ReceiveTransportMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FGuid& NodeId) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMessageTransportHandler() = default;
};
