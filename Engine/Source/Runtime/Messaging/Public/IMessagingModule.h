// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"

class IAuthorizeMessageRecipients;
class IMessageBridge;
class IMessageBus;
class IMessageTransport;

struct FMessageAddress;

/** Log category for message bus code */
DECLARE_LOG_CATEGORY_EXTERN(LogMessaging, Log, All);

/**
 * Interface for messaging modules.
 *
 * @see IMessageBridge, IMessageBus
 */
class IMessagingModule
	: public IModuleInterface
{
	using FMessageBusWeakPtr = TWeakPtr<IMessageBus, ESPMode::ThreadSafe>;

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMessageBusStartupOrShutdown, FMessageBusWeakPtr);
	
	/** Event triggered when a message bus is started. */
	virtual FOnMessageBusStartupOrShutdown& OnMessageBusStartup() = 0;

	/** Event triggered when a message bus is shutdown. */
	virtual FOnMessageBusStartupOrShutdown& OnMessageBusShutdown() = 0;

	/**
	 * Creates a new message bridge.
	 *
	 * Message bridges translate messages between a message bus and another means of
	 * message transportation, such as network sockets.
	 *
	 * @param Address The bridge's address on the message bus.
	 * @param Bus The message bus to attach the bridge to.
	 * @param Transport The message transport technology to use.
	 * @return The new message bridge, or nullptr if the bridge couldn't be created.
	 * @see CreateBus
	 */
	virtual TSharedPtr<IMessageBridge, ESPMode::ThreadSafe> CreateBridge(const FMessageAddress& Address, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& Bus, const TSharedRef<IMessageTransport, ESPMode::ThreadSafe>& Transport) = 0;

	/**
	 * Creates a new message bus.
	 *
	 * @param RecipientAuthorizer An optional recipient authorizer.
	 * @return The new message bus, or nullptr if the bus couldn't be created.
	 * @see CreateBridge
	 */
	virtual TSharedPtr<IMessageBus, ESPMode::ThreadSafe> CreateBus(const TSharedPtr<IAuthorizeMessageRecipients>& RecipientAuthorizer = nullptr) = 0;

	/**
	 * Creates a new message bus.
	 *
	 * @param Name The name of this message bus.
	 * @param RecipientAuthorizer An optional recipient authorizer.
	 * @return The new message bus, or nullptr if the bus couldn't be created.
	 * @see CreateBridge
	 */
	virtual TSharedPtr<IMessageBus, ESPMode::ThreadSafe> CreateBus(FString InName, const TSharedPtr<IAuthorizeMessageRecipients>& RecipientAuthorizer = nullptr) = 0;

	/**
	 * Gets the default message bus if it has been initialized.
	 *
	 * @return The default bus.
	 */
	virtual TSharedPtr<IMessageBus, ESPMode::ThreadSafe> GetDefaultBus() const = 0;

	/**
	 * Gets all message buses that were created by this module via the #CreateBus method
	 * that are still valid.
	 *
	 * @return All message buses.
	 */
	virtual TArray<TSharedRef<IMessageBus, ESPMode::ThreadSafe>> GetAllBuses() const = 0;

public:

	/**
	 * Gets a reference to the messaging module instance.
	 *
	 * @return A reference to the Messaging module.
	 * @todo gmp: better implementation using dependency injection.
	 */
	static IMessagingModule& Get()
	{
#if PLATFORM_IOS
        static IMessagingModule& MessageModule = FModuleManager::LoadModuleChecked<IMessagingModule>("Messaging");
        return MessageModule;
#else
        return FModuleManager::LoadModuleChecked<IMessagingModule>("Messaging");
#endif
	}

public:

	/** Virtual destructor. */
	virtual ~IMessagingModule() = default;
};
