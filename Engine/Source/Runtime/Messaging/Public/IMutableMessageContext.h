// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"
#include "Templates/SharedPointer.h"

class FName;
class FString;
class IMessageAttachment;
class UScriptStruct;

enum class EMessageScope : uint8;

struct FDateTime;
struct FMessageAddress;


/**
 * Interface for mutable message contexts.
 */
class IMutableMessageContext
	: public IMessageContext
{
public:

	/**
	 * Adds a message address to the recipient list.
	 *
	 * @param Recipient The address of the recipient to add.
	 */
	virtual void AddRecipient(const FMessageAddress& Recipient) = 0;

	/**
	 * Sets the optional message attachment.
	 *
	 * @param InAttachment The attachment to set.
	 */
	virtual void SetAttachment(const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& InAttachment) = 0;

	/**
	 * Sets the message.
	 *
	 * @param InMessage The message to set.
	 * @param InTypeInfo The message's type information.
	 */
	virtual void SetMessage(void* InMessage, UScriptStruct* InTypeInfo) = 0;

	/**
	 * Sets the date and time at which the message expires.
	 *
	 * @param InExpiration Expiration date and time.
	 */
	virtual void SetExpiration(const FDateTime& InExpiration) = 0;

	/**
	 * Sets the value of the header with the specified key.
	 *
	 * @param Key The header key.
	 * @param Value The header value.
	 */
	virtual void SetHeader(const FName& Key, const FString& Value) = 0;

	/**
	 * Sets the message scope.
	 *
	 * @param InScope The message scope.
	 */
	virtual void SetScope(EMessageScope InScope) = 0;

	/**
	 * Sets the address of the message's sender.
	 *
	 * @param InSender The message sender's address.
	 */
	virtual void SetSender(const FMessageAddress& InSender) = 0;

	/**
	 * Sets the date and time at which the message was sent.
	 *
	 * @param InTimeSent Send date and time.
	 */
	virtual void SetTimeSent(const FDateTime& InTimeSent) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMutableMessageContext() = default;
};
