// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Features/IModularFeature.h"
#include "Delegates/Delegate.h"
#include "Misc/Guid.h"
#include "Misc/Timespan.h"

struct FMessageAddress;

/** General information on transport health per Node Id. */
struct FMessageTransportStatistics
{
	/** Total number of bytes sent to the destination endpoint.*/
	uint64 TotalBytesSent = 0;

	/** Total number of bytes lost to the destination endpoint. */
	uint64 TotalBytesLost = 0;

	/** Total number of bytes received from endpoint. */
	uint64 TotalBytesReceived = 0;
	
	/** Number of bytes currently in flight waiting for. */
	uint64 BytesInflight = 0;

	/** Total number of packets sent to the destination. */
	uint64 PacketsSent = 0;

	/** Total number of packets lost to the destination. */
	uint64 PacketsLost = 0;

	/** Number of packets acknowledged. */
	uint64 PacketsAcked = 0;

	/** Number of packets received from this endpoint. */
	uint64 PacketsReceived = 0;

	/** Current packets in flight waiting for */
	uint64 PacketsInFlight = 0;

	/** The size of our sending window (as indicated by the transport).  */
	uint32 WindowSize = 0;

	/** Computed average Round-trip time to receive data from connected endpoint. */
	FTimespan AverageRTT{0};

	/** IPv4 address as a string value. */
	FString IPv4AsString;
};

/** Per-node per-message transfer statistics. */
struct FOutboundTransferStatistics
{
	/** Unique Id for the target. */
	FGuid  DestinationId;

	/** Monotonically increasing ID for each message sent across the network. */
	int32  MessageId;

	/** Total bytes to send for the given MessageId */
	uint64 BytesToSend;

	/** Total bytes sent thus far. */
	uint64 BytesSent;

	/** Total number of bytes acknowledge by the destination. */
	uint64 BytesAcknowledged;

	/** Indicate if the message is reliably sent. */
	bool bIsReliable;

	/** Indicates this was a traced packet. */
	bool bTraced;
	
	/** Indicates if the message has been completely sent. In unreliable case, it means that all bytes have been sent. In the reliable case, it means it has been fully acknowledged. */
	bool bComplete;
};

/** Per-node per-message transfer statistics. */
struct FInboundTransferStatistics
{
	/** Unique Id for the target. */
	FGuid  OriginId;

	/** Message Id that uniquely identifies the inbound message. If MessageId == -1 then this message is not reliably sent or is not segmented over multiple packets.*/
	int32  MessageId;
	
	/** Total bytes to send for the given MessageId */
	uint64 BytesToReceive;

	/** Total bytes sent thus far. */
	int32 BytesReceived;

	/** Was the message reliable. */
	bool bWasReliablySent;

	/** True, if this was a traced packet */
	bool bTraced;
	
	/** True, if we have fully received the message */
	bool bComplete;
};


DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnOutboundTransferDataUpdated, FOutboundTransferStatistics);
DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnInboundTransferDataUpdated, FInboundTransferStatistics);

/**
 * Interface for the messaging module network extension
 * Plugins or modules implementing messaging transport for MessageBus
 * can implement this modular feature to provide control on the service it provides.
 */
class INetworkMessagingExtension : public IModularFeature
{
public:
	/** The modular feature name to get the messaging extension. */
	static MESSAGING_API FName ModularFeatureName;

	/**
	 * Get the name of this messaging extension.
	 */
	virtual FName GetName() const = 0;

	/**
	 * Indicate if support is available for this extension
	 * @return true if the service can be successfully started.
	 */
	virtual bool IsSupportEnabled() const = 0;

	/**
	 * Start or restart this messaging extension service for MessageBus
	 * using its current running configuration which might include modifications to endpoints
	 * @see AddEndpoint, RemoveEndpoint
	 */
	virtual void RestartServices() = 0;

	/**
	 * The list of network addresses that we are currently listening on.  The string must be
	 * in the form of <address:port>
	 */
	virtual TArray<FString> GetListeningAddresses() const = 0;

	/**
	 * Indicates if this network messaging interface can return network statistics.
	 */
	virtual bool CanProvideNetworkStatistics() const = 0;

	/**
	 * Return the current network counters for the given Node endpoint
	 */
	virtual FMessageTransportStatistics GetLatestNetworkStatistics(FGuid NodeId) const = 0;

	/**
	 * Return the node ID corresponding to MessageAddress
	 */
	virtual FGuid GetNodeIdFromAddress(const FMessageAddress& MessageAddress) const = 0;

	/**
	 * Delegate invoked when any transmission statistics are updated for outbound nodes (sent). This delegate may get called from another thread.
	 * Please consider thread safety when receiving this delegate.
	 */
	virtual FOnOutboundTransferDataUpdated& OnOutboundTransferUpdatedFromThread() = 0;

	/**
	 * Delegate invoked when any transmission statistics are update for inbound (received). This delegate may get called from another thread.
	 * Please consider thread safety when receiving this delegate.
	 */
	virtual FOnInboundTransferDataUpdated& OnInboundTransferUpdatedFromThread() = 0;

	/**
	 * Shutdown this messaging extension services for MessageBus
	 * and remove any configuration modification.
	 * Using RestartServices after ShutdownServices will start the service with an unaltered configuration
	 * @see RestartServices
	 */
	virtual void ShutdownServices() = 0;

	/**
	 * Returns the list of internet addresses known by the transport. The addresses are in the form
	 * of <address:port>
	 */
	virtual TArray<FString> GetKnownEndpoints() const = 0;

	/**
	 * Add an endpoint to the running configuration of this messaging service
	 * This change is transient and does not modified saved configuration.
	 * @param InEndpoint the endpoint string to add to the running service, should be in the form <ipv4:port>.
	 */
	virtual void AddEndpoint(const FString& InEndpoint) = 0;

	/**
	 * Remove a static endpoint from the running configuration of the UDP messaging service
	 * This change is transient and does not modified saved configuration.
	 * @param InEndpoint the endpoint to remove from the running service, should be in the form <ipv4:port>.
	 */
	virtual void RemoveEndpoint(const FString& InEndpoint) = 0;
};
