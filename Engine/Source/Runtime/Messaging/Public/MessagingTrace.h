// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"


#if !defined(MESSAGINGTRACE_ENABLED)
#	if UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
#		define MESSAGINGTRACE_ENABLED 1
#	else
#		define MESSAGINGTRACE_ENABLED 0
#	endif
#endif


#if MESSAGINGTRACE_ENABLED

#include "Trace/Trace.h"

UE_TRACE_CHANNEL_EXTERN(MessagingChannel, MESSAGING_API);

#define SCOPED_MESSAGING_TRACE(TraceName) TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(TraceName, MessagingChannel)

#else // #if MESSAGINGTRACE_ENABLED

#define SCOPED_MESSAGING_TRACE(...)

#endif // #if MESSAGINGTRACE_ENABLED .. #else
